// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MovieSceneCapture : ModuleRules
{
	public MovieSceneCapture(ReadOnlyTargetRules Target) : base(Target)
	{
		if (Target.Type == TargetType.Editor || Target.Type == TargetType.Program)
		{
			PublicIncludePathModuleNames.Add("ImageWrapper");
			DynamicallyLoadedModuleNames.Add("ImageWrapper");
		}

		PrivateDependencyModuleNames.AddRange(
			new string[] {
                "AssetRegistry",
				"AVIWriter",
                "Core",
				"CoreUObject",
				"Engine",
				"ImageWriteQueue",
				"Json",
				"JsonUtilities",
                "RenderCore",
				"RHI",
				"Slate",
				"SlateCore",
				"AudioMixer"
			}
		);
    }
}
