// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActiveMovieSceneCaptures.h"

TUniquePtr<FActiveMovieSceneCaptures> FActiveMovieSceneCaptures::Singleton;

FActiveMovieSceneCaptures& FActiveMovieSceneCaptures::Get()
{
	if (!Singleton)
	{
		Singleton.Reset(new FActiveMovieSceneCaptures);
	}

	return *Singleton;
}

void FActiveMovieSceneCaptures::Add(UMovieSceneCapture* Capture)
{
	ActiveCaptures.AddUnique(Capture);
}

void FActiveMovieSceneCaptures::Remove(UMovieSceneCapture* Capture)
{
	ActiveCaptures.Remove(Capture);
}

void FActiveMovieSceneCaptures::Tick(float DeltaSeconds)
{
	TArray<TObjectPtr<UMovieSceneCapture>> Captures = ActiveCaptures;
	for (auto& Capture : Captures)
	{
		if (Capture->ShouldFinalize())
		{
			Capture->Finalize();
		}
		else
		{
			Capture->Tick(DeltaSeconds);
		}
	}
}

void FActiveMovieSceneCaptures::Shutdown()
{
	TArray<TObjectPtr<UMovieSceneCapture>> ActiveCapturesCopy;
	Swap(ActiveCaptures, ActiveCapturesCopy);

	for (auto& Obj : ActiveCapturesCopy)
	{
		Obj->Finalize();
	}

	Singleton.Reset();
}

void FActiveMovieSceneCaptures::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(ActiveCaptures);
}

FString FActiveMovieSceneCaptures::GetReferencerName() const
{
	return TEXT("FActiveMovieSceneCaptures");
}
