// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/GCObject.h"
#include "MovieSceneCapture.h"
#include "Tickable.h"

class FActiveMovieSceneCaptures : public FGCObject, public FTickableGameObject
{
public:
	/** Singleton access to an instance of this class */
	static FActiveMovieSceneCaptures& Get();

	/** Add a capture to this class to ensure that it will get updated */
	void Add(UMovieSceneCapture* Capture);

	/** Remove a capture from this class */
	void Remove(UMovieSceneCapture* Capture);

	/** Shutdown this class, waiting for any currently active captures to finish capturing */
	void Shutdown();

	/** We use the pending array here, as it's the most up to date */
	const TArray<UMovieSceneCapture*>& GetActiveCaptures() const { return ObjectPtrDecay(ActiveCaptures); }

private:

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;


	/** FTickableGameObject interface */
	virtual UWorld* GetTickableGameObjectWorld() const override { return ActiveCaptures.Num() != 0 ? ActiveCaptures[0]->GetWorld() : nullptr; }
	virtual bool IsTickableInEditor() const override { return false; }
	virtual bool IsTickable() const override { return ActiveCaptures.Num() != 0; }
	virtual bool IsTickableWhenPaused() const override { return false; }
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(FActiveMovieSceneCaptures, STATGROUP_Tickables); }
	virtual void Tick(float DeltaSeconds) override;

private:
	/** Private construction to enforce singleton use */
	FActiveMovieSceneCaptures() {}

	/** Array of active captures */
	TArray<TObjectPtr<UMovieSceneCapture>> ActiveCaptures;

	/** Class singleton */
	static TUniquePtr<FActiveMovieSceneCaptures> Singleton;
};
