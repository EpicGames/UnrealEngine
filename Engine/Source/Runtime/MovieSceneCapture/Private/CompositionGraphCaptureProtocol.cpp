// Copyright Epic Games, Inc. All Rights Reserved.

#include "Protocols/CompositionGraphCaptureProtocol.h"
#include "Misc/CommandLine.h"
#include "HAL/IConsoleManager.h"
#include "EngineGlobals.h"
#include "Engine/Scene.h"
#include "Materials/MaterialInterface.h"
#include "SceneView.h"
#include "Slate/SceneViewport.h"
#include "Engine/Engine.h"
#include "SceneViewExtension.h"
#include "Materials/Material.h"
#include "BufferVisualizationData.h"
#include "MovieSceneCaptureModule.h"
#include "MovieSceneCaptureSettings.h"
#include "Widgets/SWindow.h"
#include "HDRHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CompositionGraphCaptureProtocol)


struct FFrameCaptureViewExtension : public FSceneViewExtensionBase
{
	FFrameCaptureViewExtension( const FAutoRegister& AutoRegister, const TArray<FString>& InRenderPasses, bool bInCaptureFramesInHDR, int32 InHDRCompressionQuality, int32 InCaptureGamut, UMaterialInterface* InPostProcessingMaterial, bool bInDisableScreenPercentage)
		: FSceneViewExtensionBase(AutoRegister)
		, RenderPasses(InRenderPasses)
		, bNeedsCapture(false)
		, bCaptureFramesInHDR(bInCaptureFramesInHDR)
		, HDRCompressionQuality(InHDRCompressionQuality)
		, CaptureGamut(InCaptureGamut)
		, PostProcessingMaterial(InPostProcessingMaterial)
		, bDisableScreenPercentage(bInDisableScreenPercentage)
	{
		CVarDumpFrames = IConsoleManager::Get().FindConsoleVariable(TEXT("r.BufferVisualizationDumpFrames"));
		CVarDumpFramesAsHDR = IConsoleManager::Get().FindConsoleVariable(TEXT("r.BufferVisualizationDumpFramesAsHDR"));
		CVarHDRCompressionQuality = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SaveEXR.CompressionQuality"));

		RestoreDumpHDR = CVarDumpFramesAsHDR->GetInt();
		RestoreHDRCompressionQuality = CVarHDRCompressionQuality->GetInt();

		Disable();
	}

	virtual ~FFrameCaptureViewExtension()
	{
		Disable();
	}

	bool IsEnabled() const
	{
		return bNeedsCapture;
	}

	void Enable(FString&& InFilename)
	{
		OutputFilename = MoveTemp(InFilename);

		bNeedsCapture = true;

		CVarDumpFrames->Set(1);
		CVarDumpFramesAsHDR->Set(bCaptureFramesInHDR);
		CVarHDRCompressionQuality->Set(HDRCompressionQuality);
	}

	void Disable(bool bFinalize = false)
	{
		if (bNeedsCapture || bFinalize)
		{
			bNeedsCapture = false;
			if (bFinalize)
			{
				RestoreDumpHDR = 0;
				RestoreHDRCompressionQuality = 0;
			}
			CVarDumpFramesAsHDR->Set(RestoreDumpHDR);
			CVarHDRCompressionQuality->Set(RestoreHDRCompressionQuality);
			CVarDumpFrames->Set(0);
		}
	}

	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override
	{
		if (!bNeedsCapture)
		{
			return;
		}

		InView.FinalPostProcessSettings.bBufferVisualizationDumpRequired = true;
		InView.FinalPostProcessSettings.BufferVisualizationOverviewMaterials.Empty();
		InView.FinalPostProcessSettings.BufferVisualizationDumpBaseFilename = MoveTemp(OutputFilename);

		struct FIterator
		{
			FFinalPostProcessSettings& FinalPostProcessSettings;
			const TArray<FString>& RenderPasses;

			FIterator(FFinalPostProcessSettings& InFinalPostProcessSettings, const TArray<FString>& InRenderPasses)
				: FinalPostProcessSettings(InFinalPostProcessSettings), RenderPasses(InRenderPasses)
			{}

			void ProcessValue(const FString& InName, UMaterialInterface* Material, const FText& InText)
			{
				if (!RenderPasses.Num() || RenderPasses.Contains(InName) || RenderPasses.Contains(InText.ToString()))
				{
					FinalPostProcessSettings.BufferVisualizationOverviewMaterials.Add(Material);
				}
			}
		} Iterator(InView.FinalPostProcessSettings, RenderPasses);
		GetBufferVisualizationData().IterateOverAvailableMaterials(Iterator);

		if (PostProcessingMaterial)
		{
			FWeightedBlendable Blendable(1.f, PostProcessingMaterial);
			PostProcessingMaterial->OverrideBlendableSettings(InView, 1.f);
		}

		bNeedsCapture = false;
	}

	virtual void SetupViewFamily(FSceneViewFamily& InViewFamily) override
	{
		if (bDisableScreenPercentage)
		{
			// Ensure we're rendering at full size.
			InViewFamily.EngineShowFlags.ScreenPercentage = false;
		}
	}

	virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) {}
	virtual void PreRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily) {}
	virtual void PreRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView) {}

	virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext&) const override { return IsEnabled(); }

private:
	const TArray<FString>& RenderPasses;

	bool bNeedsCapture;
	FString OutputFilename;

	bool bCaptureFramesInHDR;
	int32 HDRCompressionQuality;
	int32 CaptureGamut;

	UMaterialInterface* PostProcessingMaterial;

	bool bDisableScreenPercentage;

	IConsoleVariable* CVarDumpFrames;
	IConsoleVariable* CVarDumpFramesAsHDR;
	IConsoleVariable* CVarHDRCompressionQuality;

	int32 RestoreDumpHDR;
	int32 RestoreHDRCompressionQuality;
};

bool UCompositionGraphCaptureProtocol::SetupImpl()
{
	SceneViewport = InitSettings->SceneViewport;

	FString OverrideRenderPasses;
	if (FParse::Value(FCommandLine::Get(), TEXT("-CustomRenderPasses="), OverrideRenderPasses, /*bShouldStopOnSeparator*/ false))
	{
		OverrideRenderPasses.ParseIntoArray(IncludeRenderPasses.Value, TEXT(","), true);
	}

	int32 OverrideCaptureGamut = (int32)CaptureGamut;
	FParse::Value(FCommandLine::Get(), TEXT("-CaptureGamut="), OverrideCaptureGamut);
	FParse::Value(FCommandLine::Get(), TEXT( "-HDRCompressionQuality=" ), HDRCompressionQuality);
	FParse::Bool(FCommandLine::Get(), TEXT("-CaptureFramesInHDR="), bCaptureFramesInHDR);
	FParse::Bool(FCommandLine::Get(), TEXT("-DisableScreenPercentage="), bDisableScreenPercentage);

	FString OverridePostProcessingMaterial;
	if (FParse::Value(FCommandLine::Get(), TEXT("-PostProcessingMaterial="), OverridePostProcessingMaterial, /*bShouldStopOnSeparator*/ false))
	{
		PostProcessingMaterial.SetPath(OverridePostProcessingMaterial);
	}
	PostProcessingMaterialPtr = Cast<UMaterialInterface>(PostProcessingMaterial.TryLoad());
	ViewExtension = FSceneViewExtensions::NewExtension<FFrameCaptureViewExtension>(IncludeRenderPasses.Value, bCaptureFramesInHDR, HDRCompressionQuality, OverrideCaptureGamut, PostProcessingMaterialPtr, bDisableScreenPercentage);

	EDisplayOutputFormat DisplayOutputFormat = HDRGetDefaultDisplayOutputFormat();
	EDisplayColorGamut DisplayColorGamut = HDRGetDefaultDisplayColorGamut();
	bool bHDREnabled = IsHDREnabled() && GRHISupportsHDROutput;

	if (CaptureGamut == HCGM_Linear)
	{
		DisplayColorGamut = EDisplayColorGamut::DCIP3_D65;
		DisplayOutputFormat = EDisplayOutputFormat::HDR_LinearEXR;
	}
	else
	{
		DisplayColorGamut = (EDisplayColorGamut)CaptureGamut.GetValue();
	}

	TSharedPtr<SWindow> CustomWindow = InitSettings->SceneViewport->FindWindow();
	HDRAddCustomMetaData(CustomWindow->GetNativeWindow()->GetOSWindowHandle(), DisplayOutputFormat, DisplayColorGamut, bHDREnabled);

	return true;
}

void UCompositionGraphCaptureProtocol::OnReleaseConfigImpl(FMovieSceneCaptureSettings& InSettings)
{
	// Remove {material} if it exists
	InSettings.OutputFormat = InSettings.OutputFormat.Replace(TEXT("{material}"), TEXT(""));

	// Remove .{frame} if it exists
	InSettings.OutputFormat = InSettings.OutputFormat.Replace(TEXT(".{frame}"), TEXT(""));
}

void UCompositionGraphCaptureProtocol::OnLoadConfigImpl(FMovieSceneCaptureSettings& InSettings)
{
	// Add .{frame} if it doesn't already exist
	FString OutputFormat = InSettings.OutputFormat;

	// Ensure the format string tries to always export a uniquely named frame so the file doesn't overwrite itself if the user doesn't add it.
	bool bHasFrameFormat = OutputFormat.Contains(TEXT("{frame}")) || OutputFormat.Contains(TEXT("{shot_frame}"));
	if (!bHasFrameFormat)
	{
		OutputFormat.Append(TEXT(".{frame}"));

		InSettings.OutputFormat = OutputFormat;
		UE_LOGF(LogMovieSceneCapture, Display, "Automatically appended .{frame} to the format string as specified format string did not provide a way to differentiate between frames via {frame} or {shot_frame}!");
	}

	// Add {material} if it doesn't already exist
	if (!OutputFormat.Contains(TEXT("{material}")))
	{
		int32 FramePosition = OutputFormat.Find(TEXT(".{frame}"));
		if (FramePosition != INDEX_NONE)
		{
			OutputFormat.InsertAt(FramePosition, TEXT("{material}"));
		}
		else
		{
			OutputFormat.Append(TEXT("{material}"));
		}

		InSettings.OutputFormat = OutputFormat;
	}
}

void UCompositionGraphCaptureProtocol::FinalizeImpl()
{
	TSharedPtr<SWindow> CustomWindow = InitSettings->SceneViewport->FindWindow();
	HDRRemoveCustomMetaData(CustomWindow->GetNativeWindow()->GetOSWindowHandle());

	ViewExtension->Disable(true);
}

void UCompositionGraphCaptureProtocol::CaptureFrameImpl(const FFrameMetrics& FrameMetrics)
{
	ViewExtension->Enable(GenerateFilenameImpl(FrameMetrics, TEXT("")));
}

bool UCompositionGraphCaptureProtocol::HasFinishedProcessingImpl() const
{
	return !ViewExtension->IsEnabled();
}

void UCompositionGraphCaptureProtocol::TickImpl()
{
	// If the extension is not enabled, ensure all the CVars have been reset on tick
	if (ViewExtension.IsValid() && !ViewExtension->IsEnabled())
	{
		ViewExtension->Disable();
	}
}
