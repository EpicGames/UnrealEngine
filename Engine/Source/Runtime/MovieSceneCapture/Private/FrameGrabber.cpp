// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrameGrabber.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"
#include "RenderingThread.h"
#include "RendererInterface.h"
#include "StaticBoundShaderState.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SViewport.h"
#include "RHIStaticStates.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ScreenRendering.h"
#include "PipelineStateCache.h"
#include "CommonRenderResources.h"
#include "RenderTargetPool.h"
#include "Slate/SlateViewportProvider.h"

int32 GFrameGrabberFrameLatency = 0;
static FAutoConsoleVariableRef CVarFrameGrabberFrameLatency(
	TEXT("framegrabber.framelatency"),
	GFrameGrabberFrameLatency,
	TEXT("How many frames to wait before reading back a frame. 0 frames will work but cause a performance regression due to CPU and GPU syncing up.\n"),
	ECVF_RenderThreadSafe | ECVF_Scalability
);


FViewportSurfaceReader::FViewportSurfaceReader(EPixelFormat InPixelFormat, FIntPoint InBufferSize)
{
	AvailableEvent = nullptr;
	ReadbackTexture = nullptr;
	PixelFormat = InPixelFormat;
	bQueuedForCapture = false;

	Resize(InBufferSize.X, InBufferSize.Y);
}

FViewportSurfaceReader::~FViewportSurfaceReader()
{
	BlockUntilAvailable();

	ReadbackTexture.SafeRelease();
}

void FViewportSurfaceReader::Initialize()
{
	check(!AvailableEvent);
	AvailableEvent = FPlatformProcess::GetSynchEventFromPool();
}

void FViewportSurfaceReader::Resize(uint32 Width, uint32 Height)
{
	ReadbackTexture.SafeRelease();

	FViewportSurfaceReader* This = this;
	ENQUEUE_RENDER_COMMAND(CreateCaptureFrameTexture)(
		[Width, Height, This](FRHICommandListImmediate& RHICmdList)
		{
			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create2D(TEXT("FViewportSurfaceReader_ReadbackTexture"), Width, Height, This->PixelFormat)
				.SetFlags(ETextureCreateFlags::CPUReadback);

			This->ReadbackTexture = RHICmdList.CreateTexture(Desc);
		});
}

void FViewportSurfaceReader::BlockUntilAvailable()
{
	if (AvailableEvent)
	{
		AvailableEvent->Wait(~0);

		FPlatformProcess::ReturnSynchEventToPool(AvailableEvent);
		AvailableEvent = nullptr;
	}
}

void FViewportSurfaceReader::Reset()
{
	if (AvailableEvent)
	{
		AvailableEvent->Trigger();
	}
	BlockUntilAvailable();
	bQueuedForCapture = false;
}

void FViewportSurfaceReader::ResolveRenderTarget(FViewportSurfaceReader* RenderToReadback, const FTextureRHIRef& SourceBackBuffer, TFunction<void(FColor*, int32, int32)> Callback)
{
	static const FName RendererModuleName( "Renderer" );
	// @todo: JIRA UE-41879 and UE-43829 - added defensive guards against memory trampling on this render command to try and ascertain why it occasionally crashes
	uint32 MemoryGuard1 = 0xaffec7ed;

	// Load the renderermodule on the main thread, as the module manager is not thread-safe, and copy the ptr into the render command, along with 'this' (which is protected by BlockUntilAvailable in ~FViewportSurfaceReader())
	IRendererModule* RendererModule = &FModuleManager::GetModuleChecked<IRendererModule>(RendererModuleName);

	uint32 MemoryGuard2 = 0xaffec7ed;
	IRendererModule* RendererModuleDebug = RendererModule;

	bQueuedForCapture = true;

	{
		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

		const FIntPoint TargetSize(ReadbackTexture->GetSizeX(), ReadbackTexture->GetSizeY());

		FPooledRenderTargetDesc OutputDesc = FPooledRenderTargetDesc::Create2DDesc(
			TargetSize,
			ReadbackTexture->GetFormat(),
			FClearValueBinding::None,
			TexCreate_None,
			TexCreate_RenderTargetable,
			false);

		TRefCountPtr<IPooledRenderTarget> ResampleTexturePooledRenderTarget;
		GRenderTargetPool.FindFreeElement(RHICmdList, OutputDesc, ResampleTexturePooledRenderTarget, TEXT("ResampleTexture"));
		check(ResampleTexturePooledRenderTarget);

		FRHITexture* DestRenderTarget = ResampleTexturePooledRenderTarget->GetRHI();

		FRHIRenderPassInfo RPInfo(DestRenderTarget, ERenderTargetActions::Load_Store);
		RHICmdList.BeginRenderPass(RPInfo, TEXT("FrameGrabberResolveRenderTarget"));
		{
			RHICmdList.SetViewport(0, 0, 0.0f, TargetSize.X, TargetSize.Y, 1.0f);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
			GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

			const ERHIFeatureLevel::Type FeatureLevel = GMaxRHIFeatureLevel;

			FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(FeatureLevel);
			TShaderMapRef<FScreenVS> VertexShader(ShaderMap);
			TShaderMapRef<FScreenPS> PixelShader(ShaderMap);

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			const bool bIsSourceBackBufferSameAsWindowSize = SourceBackBuffer->GetSizeX() == WindowSize.X && SourceBackBuffer->GetSizeY() == WindowSize.Y;
			const bool bIsSourceBackBufferSameAsTargetSize = TargetSize.X == SourceBackBuffer->GetSizeX() && TargetSize.Y == SourceBackBuffer->GetSizeY();

			FRHISamplerState* SamplerState = (bIsSourceBackBufferSameAsWindowSize || bIsSourceBackBufferSameAsTargetSize) ? TStaticSamplerState<SF_Point>::GetRHI() : TStaticSamplerState<SF_Bilinear>::GetRHI();

			SetShaderParametersLegacyPS(RHICmdList, PixelShader, SamplerState, SourceBackBuffer);

			float U = float(CaptureRect.Min.X) / float(SourceBackBuffer->GetSizeX());
			float V = float(CaptureRect.Min.Y) / float(SourceBackBuffer->GetSizeY());
			float SizeU = float(CaptureRect.Max.X) / float(SourceBackBuffer->GetSizeX()) - U;
			float SizeV = float(CaptureRect.Max.Y) / float(SourceBackBuffer->GetSizeY()) - V;

			RendererModule->DrawRectangle(
				RHICmdList,
				0, 0,									// Dest X, Y
				TargetSize.X,							// Dest Width
				TargetSize.Y,							// Dest Height
				U, V,									// Source U, V
				SizeU, SizeV,							// Source USize, VSize
				CaptureRect.Max - CaptureRect.Min,		// Target buffer size
				FIntPoint(1, 1),						// Source texture size
				VertexShader,
				EDRF_Default);
		}
		RHICmdList.EndRenderPass();
		FGPUFenceRHIRef GPUFence = RHICreateGPUFence(TEXT("FrameGrabberReadbackFence"));
		TransitionAndCopyTexture(RHICmdList, DestRenderTarget, ReadbackTexture, {});
		RHICmdList.WriteGPUFence(GPUFence);

		if (RenderToReadback)
		{
			void* ColorDataBuffer = nullptr;

			int32 Width = 0, Height = 0;
			RHICmdList.MapStagingSurface(RenderToReadback->ReadbackTexture, GPUFence.GetReference(), ColorDataBuffer, Width, Height);

			Callback((FColor*)ColorDataBuffer, Width, Height);

			RHICmdList.UnmapStagingSurface(RenderToReadback->ReadbackTexture);
			RenderToReadback->AvailableEvent->Trigger();
		}
	};
}

FFrameGrabber::FFrameGrabber(TSharedRef<FSceneViewport> Viewport, FIntPoint DesiredBufferSize, EPixelFormat InPixelFormat, uint32 NumSurfaces)
{
	State = EFrameGrabberState::Inactive;

	TargetSize = DesiredBufferSize;

	CurrentFrameIndex = 0;
	TargetWindowPtr = nullptr;

	check(NumSurfaces != 0);

	FIntRect CaptureRect(0,0,Viewport->GetSize().X, Viewport->GetSize().Y);
	FIntPoint WindowSize(0, 0);

	// Set up the capture rectangle
	TSharedPtr<SViewport> ViewportWidget = Viewport->GetViewportWidget().Pin();
	if (ViewportWidget.IsValid())
	{
		TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(ViewportWidget.ToSharedRef());
		if (Window.IsValid())
		{
			TargetWindowPtr = Window.Get();
			FGeometry InnerWindowGeometry = Window->GetWindowGeometryInWindow();
			
			// Find the widget path relative to the window
			FArrangedChildren JustWindow(EVisibility::Visible);
			JustWindow.AddWidget(FArrangedWidget(Window.ToSharedRef(), InnerWindowGeometry));

			FWidgetPath WidgetPath(Window.ToSharedRef(), JustWindow);
			if (WidgetPath.ExtendPathTo(FWidgetMatcher(ViewportWidget.ToSharedRef()), EVisibility::Visible))
			{
				FArrangedWidget ArrangedWidget = WidgetPath.FindArrangedWidget(ViewportWidget.ToSharedRef()).Get(FArrangedWidget::GetNullWidget());

				FVector2D Position = ArrangedWidget.Geometry.GetAbsolutePosition();
				FVector2D Size = ArrangedWidget.Geometry.GetAbsoluteSize();

				CaptureRect = FIntRect(
					Position.X,
					Position.Y,
					Position.X + Size.X,
					Position.Y + Size.Y);

				FVector2D AbsoluteSize = InnerWindowGeometry.GetAbsoluteSize();
				WindowSize = FIntPoint(AbsoluteSize.X, AbsoluteSize.Y);

			}
		}
	}

	// This can never be reallocated
	Surfaces.Reserve(NumSurfaces);
	for (uint32 Index = 0; Index < NumSurfaces; ++Index)
	{
		Surfaces.Emplace(InPixelFormat, DesiredBufferSize);
		Surfaces.Last().Surface.SetCaptureRect(CaptureRect);
		Surfaces.Last().Surface.SetWindowSize(WindowSize);
	}

	FrameGrabLatency = GFrameGrabberFrameLatency;

	// Ensure textures are setup
	FlushRenderingCommands();
}

FFrameGrabber::~FFrameGrabber()
{
	if (OnBackBufferReadyToPresent.IsValid() && FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetRenderer()->OnBackBufferReadyToPresent().Remove(OnBackBufferReadyToPresent);
	}

	if (OutstandingFrameCount.GetValue() > 0)
	{
		FlushRenderingCommands();
	}
}

void FFrameGrabber::StartCapturingFrames()
{
	if (!ensure(State == EFrameGrabberState::Inactive))
	{
		return;
	}

	State = EFrameGrabberState::Active;

	OnBackBufferReadyToPresent = FSlateApplication::Get().GetRenderer()->OnBackBufferReadyToPresent().AddRaw(this, &FFrameGrabber::OnBackBufferReadyToPresentCallback);
}

bool FFrameGrabber::IsCapturingFrames() const
{
	return State == EFrameGrabberState::Active;
}

void FFrameGrabber::CaptureThisFrame(FFramePayloadPtr Payload)
{
	if (!ensure(State == EFrameGrabberState::Active))
	{
		return;
	}

	OutstandingFrameCount.Increment();

	ENQUEUE_RENDER_COMMAND(FrameGrabber_AppendFramePayload)(
		[this, Payload](FRHICommandListImmediate& RHICmdList)
		{
			this->RenderThread_PendingFramePayloads.Add(Payload);
		}
	);
}

void FFrameGrabber::StopCapturingFrames()
{
	if (!ensure(State == EFrameGrabberState::Active))
	{
		return;
	}

	State = EFrameGrabberState::PendingShutdown;
}

void FFrameGrabber::Shutdown()
{
	State = EFrameGrabberState::Inactive;

	for (FResolveSurface& Surface : Surfaces)
	{
		Surface.Surface.BlockUntilAvailable();
	}

	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetRenderer()->OnBackBufferReadyToPresent().Remove(OnBackBufferReadyToPresent);
	}
	OnBackBufferReadyToPresent = FDelegateHandle();
}

bool FFrameGrabber::HasOutstandingFrames() const
{
	FScopeLock Lock(&CapturedFramesMutex);

	// Check whether we have any outstanding frames while we have the array locked, to prevent race condition
	return OutstandingFrameCount.GetValue() || CapturedFrames.Num();
}

TArray<FCapturedFrameData> FFrameGrabber::GetCapturedFrames()
{
	TArray<FCapturedFrameData> ReturnFrames;

	bool bShouldStop = false;
	{
		FScopeLock Lock(&CapturedFramesMutex);
		Swap(ReturnFrames, CapturedFrames);
		CapturedFrames.Reset();

		// Check whether we have any outstanding frames while we have the array locked, to prevent race condition
		if (State == EFrameGrabberState::PendingShutdown)
		{
			bShouldStop = OutstandingFrameCount.GetValue() == 0;
		}
	}

	if (bShouldStop)
	{
		Shutdown();
	}

	return ReturnFrames;
}

void FFrameGrabber::OnBackBufferReadyToPresentCallback(SWindow& SlateWindow, ISlateViewportProvider& ViewportProvider)
{
	// We only care about our own Slate window
	if (&SlateWindow != TargetWindowPtr)
	{
		return;
	}

	ensure(IsInRenderingThread());

	FFramePayloadPtr Payload;

	{
		if (!RenderThread_PendingFramePayloads.Num())
		{
			// No frames to capture
			return;
		}

		Payload = RenderThread_PendingFramePayloads[0];
		RenderThread_PendingFramePayloads.RemoveAt(0, EAllowShrinking::No);
	}

	if (FrameGrabLatency != GFrameGrabberFrameLatency)
	{
		FlushRenderingCommands();
		for (FResolveSurface& Surface : Surfaces)
		{
			Surface.Surface.Reset();
			CurrentFrameIndex = 0;
		}
		FrameGrabLatency = GFrameGrabberFrameLatency;
	}

	const int32 PrevCaptureIndexOffset = FMath::Clamp(FrameGrabLatency, 0, Surfaces.Num() - 1);
	const int32 ThisCaptureIndex = CurrentFrameIndex;
	const int32 PrevCaptureIndex = (CurrentFrameIndex - PrevCaptureIndexOffset) < 0 ? Surfaces.Num() - (PrevCaptureIndexOffset - CurrentFrameIndex) : (CurrentFrameIndex - PrevCaptureIndexOffset);

	FResolveSurface* NextFrameTarget = &Surfaces[ThisCaptureIndex];
	NextFrameTarget->Surface.BlockUntilAvailable();

	NextFrameTarget->Surface.Initialize();
	NextFrameTarget->Payload = Payload;

	FViewportSurfaceReader* PrevFrameTarget = &Surfaces[PrevCaptureIndex].Surface;

	//If the latency is 0, then we are asking to readback the frame we are currently queuing immediately. 
	if (!PrevFrameTarget->WasEverQueued() && (PrevCaptureIndexOffset > 0))
	{
		PrevFrameTarget = nullptr;
	}

	Surfaces[ThisCaptureIndex].Surface.ResolveRenderTarget(PrevFrameTarget, ViewportProvider.GetBackBufferResource(), [this, ThisCaptureIndex](FColor* ColorBuffer, int32 Width, int32 Height) {
		// Handle the frame
		OnFrameReady(ThisCaptureIndex, ColorBuffer, Width, Height);
	});

	CurrentFrameIndex = (CurrentFrameIndex+1) % Surfaces.Num();
}

void FFrameGrabber::OnFrameReady(int32 BufferIndex, FColor* ColorBuffer, int32 Width, int32 Height)
{
	if (!ensure(ColorBuffer))
	{
		OutstandingFrameCount.Decrement();
		return;
	}
	
	const FResolveSurface& Surface = Surfaces[BufferIndex];

	bool bExecuteDefaultGrabber = true;
	if (Surface.Payload.IsValid())
	{
		bExecuteDefaultGrabber = Surface.Payload->OnFrameReady_RenderThread(ColorBuffer, FIntPoint(Width, Height), TargetSize);
	}

	if (bExecuteDefaultGrabber)
	{
		FCapturedFrameData ResolvedFrameData(TargetSize, Surface.Payload);

		ResolvedFrameData.ColorBuffer.InsertUninitialized(0, TargetSize.X * TargetSize.Y);
		FColor* Dest = &ResolvedFrameData.ColorBuffer[0];

		const int32 MaxWidth = FMath::Min(TargetSize.X, Width);
		for (int32 Row = 0; Row < FMath::Min(Height, TargetSize.Y); ++Row)
		{
			FMemory::Memcpy(Dest, ColorBuffer, sizeof(FColor)*MaxWidth);
			ColorBuffer += Width;
			Dest += MaxWidth;
		}

		{
			FScopeLock Lock(&CapturedFramesMutex);
			CapturedFrames.Add(MoveTemp(ResolvedFrameData));
		}
	}

	OutstandingFrameCount.Decrement();
}
