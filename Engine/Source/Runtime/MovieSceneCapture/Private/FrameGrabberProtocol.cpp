// Copyright Epic Games, Inc. All Rights Reserved.

#include "Protocols/FrameGrabberProtocol.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FrameGrabberProtocol)

bool UFrameGrabberProtocol::HasFinishedProcessingImpl() const
{
	return FrameGrabber.IsValid() && !FrameGrabber->HasOutstandingFrames();
}

bool UFrameGrabberProtocol::SetupImpl()
{
	// We'll use our own grabber to capture the entire viewport
	FrameGrabber.Reset(new FFrameGrabber(InitSettings->SceneViewport.ToSharedRef(), InitSettings->DesiredSize, DesiredPixelFormat, RingBufferSize));
	FrameGrabber->StartCapturingFrames();
	return true;
}

void UFrameGrabberProtocol::BeginFinalizeImpl()
{
	FrameGrabber->StopCapturingFrames();
}

void UFrameGrabberProtocol::CaptureFrameImpl(const FFrameMetrics& FrameMetrics)
{
	if (FrameGrabber.IsValid())
	{
		FrameGrabber->CaptureThisFrame(GetFramePayload(FrameMetrics));
	}
}

void UFrameGrabberProtocol::TickImpl()
{
	if (FrameGrabber.IsValid())
	{
		TArray<FCapturedFrameData> CapturedFrames = FrameGrabber->GetCapturedFrames();

		for (FCapturedFrameData& Frame : CapturedFrames)
		{
			ProcessFrame(MoveTemp(Frame));
		}
	}
}

void UFrameGrabberProtocol::FinalizeImpl()
{
	if (FrameGrabber.IsValid())
	{
		FrameGrabber->Shutdown();
		FrameGrabber.Reset();
	}
}

