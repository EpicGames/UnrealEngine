// Copyright Epic Games, Inc. All Rights Reserved.

#include "Protocols/ImageSequenceProtocol.h"

#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "HAL/RunnableThread.h"
#include "Misc/ScopeLock.h"
#include "Misc/FeedbackContext.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/DefaultValueHelper.h"
#include "Modules/ModuleManager.h"
#include "Async/Future.h"
#include "Async/Async.h"
#include "Templates/Casts.h"
#include "MovieSceneCaptureModule.h"
#include "MovieSceneCaptureSettings.h"
#include "ImageWriteQueue.h"
#include "Widgets/SWindow.h"
#include "HDRHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImageSequenceProtocol)

struct FImageFrameData : IFramePayload
{
	FString Filename;
};

UImageSequenceProtocol::UImageSequenceProtocol(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	Format = EImageFormat::BMP;
	ImageWriteQueue = nullptr;
}

void UImageSequenceProtocol::OnLoadConfigImpl(FMovieSceneCaptureSettings& InSettings)
{
	// Add .{frame} if it doesn't already exist
	FString OutputFormat = InSettings.OutputFormat;

	// Ensure the format string tries to always export a uniquely named frame so the file doesn't overwrite itself if the user doesn't add it.
	bool bHasFrameFormat = OutputFormat.Contains(TEXT("{frame}")) || OutputFormat.Contains(TEXT("{shot_frame}"));
	if (!bHasFrameFormat)
	{
		OutputFormat.Append(TEXT(".{frame}"));
		InSettings.OutputFormat = OutputFormat;

		UE_LOGF(LogMovieSceneCapture, Display, "Automatically appended .{frame} to the format string as specified format string did not provide a way to differentiate between frames via {frame} or {shot_frame}!");
	}

	Super::OnLoadConfigImpl(InSettings);
}

void UImageSequenceProtocol::OnReleaseConfigImpl(FMovieSceneCaptureSettings& InSettings)
{
	// Remove .{frame} if it exists. The "." before the {frame} is intentional because some media players denote frame numbers separated by "."
	InSettings.OutputFormat = InSettings.OutputFormat.Replace(TEXT(".{frame}"), TEXT(""));

	Super::OnReleaseConfigImpl(InSettings);
}

bool UImageSequenceProtocol::SetupImpl()
{
	ImageWriteQueue = &FModuleManager::Get().LoadModuleChecked<IImageWriteQueueModule>("ImageWriteQueue").GetWriteQueue();
	FinalizeFence = TFuture<void>();

	return Super::SetupImpl();
}

bool UImageSequenceProtocol::HasFinishedProcessingImpl() const
{
	return Super::HasFinishedProcessingImpl() && (!FinalizeFence.IsValid() || FinalizeFence.WaitFor(0));
}

void UImageSequenceProtocol::BeginFinalizeImpl()
{
	FinalizeFence = ImageWriteQueue->CreateFence();
}

void UImageSequenceProtocol::FinalizeImpl()
{
	if (FinalizeFence.IsValid())
	{
		double StartTime = FPlatformTime::Seconds();

		FScopedSlowTask SlowTask(0, NSLOCTEXT("ImageSequenceProtocol", "Finalizing", "Finalizing write operations..."));
		SlowTask.MakeDialogDelayed(.1f, true, true);

		FTimespan HalfSecond = FTimespan::FromSeconds(0.5);
		while ( !GWarn->ReceivedUserCancel() && !FinalizeFence.WaitFor(HalfSecond) )
		{
			// Tick the slow task
			SlowTask.EnterProgressFrame(0);
		}
	}

	Super::FinalizeImpl();
}

FFramePayloadPtr UImageSequenceProtocol::GetFramePayload(const FFrameMetrics& FrameMetrics)
{
	TSharedRef<FImageFrameData, ESPMode::ThreadSafe> FrameData = MakeShareable(new FImageFrameData);

	const TCHAR* Extension = TEXT("");
	switch(Format)
	{
	case EImageFormat::BMP:		Extension = TEXT(".bmp"); break;
	case EImageFormat::PNG:		Extension = TEXT(".png"); break;
	case EImageFormat::JPEG:	Extension = TEXT(".jpg"); break;
	case EImageFormat::EXR:		Extension = TEXT(".exr"); break;
	}

	FrameData->Filename = GenerateFilenameImpl(FrameMetrics, Extension);
	EnsureFileWritableImpl(FrameData->Filename);

	// Add our custom formatting rules as well
	// @todo: document these on the tooltip?
	FrameData->Filename = FString::Format(*FrameData->Filename, StringFormatMap);

	return FrameData;
}

void UImageSequenceProtocol::ProcessFrame(FCapturedFrameData Frame)
{
	check(Frame.ColorBuffer.Num() >= Frame.BufferSize.X * Frame.BufferSize.Y);

	TUniquePtr<FImageWriteTask> ImageTask = MakeUnique<FImageWriteTask>();

	// Move the color buffer into a raw image data container that we can pass to the write queue
	ImageTask->PixelData = MakeUnique<TImagePixelData<FColor>>(Frame.BufferSize, TArray64<FColor>(MoveTemp(Frame.ColorBuffer)));
	if (Format == EImageFormat::PNG)
	{
		// Always write full alpha for PNGs
		ImageTask->AddPreProcessorToSetAlphaOpaque();
	}

	switch (Format)
	{
	case EImageFormat::EXR:
	case EImageFormat::PNG:
	case EImageFormat::BMP:
	case EImageFormat::JPEG:
		ImageTask->Format = Format;
		break;

	default:
		check(false);
		break;
	}

	ImageTask->CompressionQuality = GetCompressionQuality();
	ImageTask->Filename = Frame.GetPayload<FImageFrameData>()->Filename;

	ImageWriteQueue->Enqueue(MoveTemp(ImageTask));
}

void UImageSequenceProtocol::AddFormatMappingsImpl(TMap<FString, FStringFormatArg>& FormatMappings) const
{
	FormatMappings.Add(TEXT("quality"), TEXT(""));
}

bool UCompressedImageSequenceProtocol::SetupImpl()
{
	FParse::Value( FCommandLine::Get(), TEXT( "-MovieQuality=" ), CompressionQuality );
	CompressionQuality = FMath::Clamp<int32>(CompressionQuality, 1, 100);

	return Super::SetupImpl();
}

void UCompressedImageSequenceProtocol::AddFormatMappingsImpl(TMap<FString, FStringFormatArg>& FormatMappings) const
{
	FormatMappings.Add(TEXT("quality"), CompressionQuality);
}

UImageSequenceProtocol_EXR::UImageSequenceProtocol_EXR(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	Format = EImageFormat::EXR;
	bCompressed = false;
	CaptureGamut = HCGM_Rec709;
}

bool UImageSequenceProtocol_EXR::SetupImpl()
{
	{
		int32 OverrideCaptureGamut = (int32)CaptureGamut;
		FString CaptureGamutString;

		if (FParse::Value(FCommandLine::Get(), TEXT("-CaptureGamut="), CaptureGamutString))
		{
			if (!FDefaultValueHelper::ParseInt(CaptureGamutString, OverrideCaptureGamut))
			{
				OverrideCaptureGamut = StaticEnum<EHDRCaptureGamut>()->GetValueByName(FName(*CaptureGamutString));
			}
			// Invalid CaptureGamut will crash (see UImageSequenceProtocol_EXR::AddFormatMappingsImpl), so only set if valid.
			if (OverrideCaptureGamut > INDEX_NONE && OverrideCaptureGamut < EHDRCaptureGamut::HCGM_MAX)
			{
				CaptureGamut = (EHDRCaptureGamut)OverrideCaptureGamut;
			}
			else
			{
				UE_LOGF(LogMovieSceneCapture, Warning, "The value for the command -CaptureGamut is invalid, using default value instead!")
			}
		}
	}

	int32 HDRCompressionQuality = 0;
	if ( FParse::Value( FCommandLine::Get(), TEXT( "-HDRCompressionQuality=" ), HDRCompressionQuality ) )
	{
		bCompressed = HDRCompressionQuality != (int32)EImageCompressionQuality::Uncompressed;
	}

	EDisplayOutputFormat DisplayOutputFormat = HDRGetDefaultDisplayOutputFormat();
	EDisplayColorGamut DisplayColorGamut = HDRGetDefaultDisplayColorGamut();
	bool bHDREnabled = IsHDREnabled() && GRHISupportsHDROutput;

	if (CaptureGamut == HCGM_Linear)
	{
		DisplayColorGamut = EDisplayColorGamut::DCIP3_D65;
		DisplayOutputFormat = EDisplayOutputFormat::HDR_LinearEXR;
	}
	else
	{
		DisplayColorGamut = (EDisplayColorGamut)CaptureGamut.GetValue();
	}

	TSharedPtr<SWindow> CustomWindow = InitSettings->SceneViewport->FindWindow();
	HDRAddCustomMetaData(CustomWindow->GetNativeWindow()->GetOSWindowHandle(), DisplayOutputFormat, DisplayColorGamut, bHDREnabled);

	return Super::SetupImpl();
}

void UImageSequenceProtocol_EXR::FinalizeImpl()
{
	TSharedPtr<SWindow> CustomWindow = InitSettings->SceneViewport->FindWindow();
	HDRRemoveCustomMetaData(CustomWindow->GetNativeWindow()->GetOSWindowHandle());

	Super::FinalizeImpl();
}

void UImageSequenceProtocol_EXR::AddFormatMappingsImpl(TMap<FString, FStringFormatArg>& FormatMappings) const
{
	FormatMappings.Add(TEXT("quality"), bCompressed ? TEXT("Compressed") : TEXT("Uncompressed"));

	const TCHAR* GamutString = TEXT("");
	switch (CaptureGamut)
	{
		case HCGM_Rec709:  GamutString = TEXT("sRGB"); break;
		case HCGM_P3DCI:   GamutString = TEXT("P3D65"); break;
		case HCGM_Rec2020: GamutString = TEXT("Rec2020"); break;
		case HCGM_ACES:    GamutString = TEXT("ACES"); break;
		case HCGM_ACEScg:  GamutString = TEXT("ACEScg"); break;
		case HCGM_Linear:  GamutString = TEXT("Linear"); break;
		default: check(false); break;
	}
	FormatMappings.Add(TEXT("gamut"), GamutString);
}

