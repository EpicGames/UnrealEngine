// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelCapture.h"
#include "UObject/LazyObjectPtr.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelCapture)

ULevelCapture::ULevelCapture(const FObjectInitializer& Init)
	: Super(Init)
{
	bAutoStartCapture = true;
}

void ULevelCapture::SetPrerequisiteActor(AActor* InPrerequisiteActor)
{
	PrerequisiteActorId = TLazyObjectPtr<>(InPrerequisiteActor).GetUniqueID().GetGuid();
}

void ULevelCapture::Initialize(TSharedPtr<FSceneViewport> InViewport, int32 PIEInstance)
{
	CaptureStrategy = MakeShareable(new FFixedTimeStepCaptureStrategy(Settings.GetFrameRate()));
	CaptureStrategy->OnInitialize();
	Super::Initialize(InViewport, PIEInstance);

	PIECaptureInstance = PIEInstance;

	if (bAutoStartCapture)
	{
		StartCapture();
	}
}

void ULevelCapture::OnTick(float DeltaSeconds)
{
	AActor* Actor = PrerequisiteActor.Get();
	if (!Actor)
	{
		// We need to look the actor up
		TLazyObjectPtr<AActor> LazyActor;
		if (PIECaptureInstance != -1)
		{
			LazyActor = FUniqueObjectGuid(PrerequisiteActorId).FixupForPIE(PIECaptureInstance);
		}
		else
		{
			LazyActor = FUniqueObjectGuid(PrerequisiteActorId);
		}

		PrerequisiteActor = Actor = LazyActor.Get();
	}

	if (Actor && Actor->GetWorld() && !Actor->GetWorld()->HasBegunPlay())
	{
		return;
	}

	if (!PrerequisiteActorId.IsValid() || (Actor && Actor->HasActorBegunPlay()))
	{
		CaptureThisFrame(DeltaSeconds);
	}
}

