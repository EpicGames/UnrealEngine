// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneCapture.h"
#include "AssetRegistry/AssetData.h"
#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
#include "Engine/World.h"
#include "Slate/SceneViewport.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "ActiveMovieSceneCaptures.h"
#include "JsonObjectConverter.h"
#include "Misc/RemoteConfigIni.h"
#include "MovieSceneCaptureModule.h"
#include "Engine/GameViewportClient.h"
#include "Protocols/VideoCaptureProtocol.h"
#include "Slate/SceneViewport.h"
#include "Engine/Engine.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "UObject/UObjectIterator.h"
#include "Protocols/AudioCaptureProtocol.h"
#include "Misc/DateTime.h"
#include "RenderUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCapture)

#define LOCTEXT_NAMESPACE "MovieSceneCapture"

const FName UMovieSceneCapture::MovieSceneCaptureUIName = FName("MovieSceneCaptureUIInstance");

TArray<UClass*> FindAllCaptureProtocolClasses()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Retrieve all blueprint classes 
	TArray<FAssetData> BlueprintList;

	FARFilter Filter;
	Filter.ClassPaths.Add(UMovieSceneCaptureProtocolBase::StaticClass()->GetClassPathName());

	// Include any Blueprint based objects as well, this includes things like Blutilities, UMG, and GameplayAbility objects
	Filter.bRecursiveClasses = true;
	AssetRegistryModule.Get().GetAssets(Filter, BlueprintList);

	TArray<UClass*> Classes;

	for (const FAssetData& Data : BlueprintList)
	{
		UClass* Class = Data.GetClass();
		if (Class)
		{
			Classes.Add(Class);
		}
	}

	for (TObjectIterator<UClass> ClassIterator; ClassIterator; ++ClassIterator)
	{
		if (ClassIterator->IsChildOf(UMovieSceneCaptureProtocolBase::StaticClass()) && !ClassIterator->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
		{
			Classes.Add(*ClassIterator);
		}
	}

	return Classes;
}

struct FUniqueMovieSceneCaptureHandle : FMovieSceneCaptureHandle
{
	FUniqueMovieSceneCaptureHandle()
	{
		/// Start IDs at index 1 since 0 is deemed invalid
		static uint32 Unique = 1;
		ID = Unique++;
	}
};

FMovieSceneCaptureSettings::FMovieSceneCaptureSettings()
	: Resolution(1280, 720)
{
	OutputDirectory.Path = FPaths::VideoCaptureDir();
	FPaths::MakePlatformFilename( OutputDirectory.Path );

	bOverwriteExisting = false;
	bUseRelativeFrameNumbers = false;
	HandleFrames = 0;
	GameModeOverride = nullptr;
	OutputFormat = TEXT("{world}");
	bUseCustomFrameRate = false;
	CustomFrameRate = FFrameRate(24, 1);
	FrameRate = FFrameRate(24, 1);
	ZeroPadFrameNumbers = 4;
	bEnableTextureStreaming = false;
	bCinematicEngineScalability = true;
	bCinematicMode = true;
	bAllowMovement = false;
	bAllowTurning = false;
	bShowPlayer = false;
	bShowHUD = false;
	bUsePathTracer = false;
	PathTracerSamplePerPixel = 16;

#if PLATFORM_MAC
	MovieExtension = TEXT(".mov");
#elif PLATFORM_UNIX
	MovieExtension = TEXT(".unsupp");
#else
	MovieExtension = TEXT(".avi");
#endif
}

UMovieSceneCapture::UMovieSceneCapture(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
	TArray<FString> Tokens, Switches;
	FCommandLine::Parse( FCommandLine::Get(), Tokens, Switches );
	for (auto& Switch : Switches)
	{
		InheritedCommandLineArguments.AppendChar('-');
		InheritedCommandLineArguments.Append(Switch);
		InheritedCommandLineArguments.AppendChar(' ');
	}
	
	AdditionalCommandLineArguments += TEXT("-NOSCREENMESSAGES");

	Handle = FUniqueMovieSceneCaptureHandle();

	bCapturing = false;
	FrameNumberOffset = 0;
	ImageCaptureProtocolType = UVideoCaptureProtocol::StaticClass();
	AudioCaptureProtocolType = UNullAudioCaptureProtocol::StaticClass();
}

void UMovieSceneCapture::PostInitProperties()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		InitializeCaptureProtocols();
	}

	Super::PostInitProperties();
}

void UMovieSceneCapture::SetImageCaptureProtocolType(TSubclassOf<UMovieSceneCaptureProtocolBase> ProtocolType)
{
	ImageCaptureProtocolType = ProtocolType.Get();
	InitializeCaptureProtocols();
}

void UMovieSceneCapture::SetAudioCaptureProtocolType(TSubclassOf<UMovieSceneCaptureProtocolBase> ProtocolType)
{
	AudioCaptureProtocolType = ProtocolType.Get();
	InitializeCaptureProtocols();
}

void UMovieSceneCapture::ForciblyReinitializeCaptureProtocols()
{
	UClass* ImageProtocolType = ImageCaptureProtocolType.TryLoadClass<UMovieSceneImageCaptureProtocolBase>();
	UClass* AudioProtocolType = AudioCaptureProtocolType.TryLoadClass<UMovieSceneAudioCaptureProtocolBase>();

	if (ImageCaptureProtocol)
	{
		// Release the protocol since we know now that it's either not needed (the type is None), or it's the wrong type
		ImageCaptureProtocol->OnReleaseConfig(Settings);
		FName UniqueDeadName = MakeUniqueObjectName(GetTransientPackage(), UMovieSceneImageCaptureProtocolBase::StaticClass(), "ImageCaptureProtocol_DEAD");
		ImageCaptureProtocol->Rename(*UniqueDeadName.ToString(), GetTransientPackage());
		ImageCaptureProtocol = nullptr;
	}
	
	if (AudioCaptureProtocol)
	{
		// Release the protocol since we know now that it's either not needed (the type is None), or it's the wrong type
		AudioCaptureProtocol->OnReleaseConfig(Settings);
		FName UniqueDeadName = MakeUniqueObjectName(GetTransientPackage(), UMovieSceneAudioCaptureProtocolBase::StaticClass(), "AudioCaptureProtocol_DEAD");
		AudioCaptureProtocol->Rename(*UniqueDeadName.ToString(), GetTransientPackage());
		AudioCaptureProtocol = nullptr;
	}

	if (ImageProtocolType)
	{
		FString ProtocolName = GetName() + TEXT("_ImageProtocol");
		ImageCaptureProtocol = NewObject<UMovieSceneImageCaptureProtocolBase>(this, ImageProtocolType, *ProtocolName);
		if (ImageCaptureProtocol)
		{
			ImageCaptureProtocol->LoadConfig();
			ImageCaptureProtocol->OnLoadConfig(Settings);
		}
		
#if WITH_EDITOR
		FPropertyChangedEvent PropertyChangedEvent(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMovieSceneCapture, ImageCaptureProtocol)), EPropertyChangeType::ValueSet);
		PostEditChangeProperty(PropertyChangedEvent);
#endif
	}
	
	if (AudioProtocolType)
	{
		FString ProtocolName = GetName() + TEXT("_AudioProtocol");
		AudioCaptureProtocol = NewObject<UMovieSceneAudioCaptureProtocolBase>(this, AudioProtocolType, *ProtocolName);
		if (AudioCaptureProtocol)
		{
			AudioCaptureProtocol->LoadConfig();
			AudioCaptureProtocol->OnLoadConfig(Settings);
		}
		
#if WITH_EDITOR
		FPropertyChangedEvent PropertyChangedEvent(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMovieSceneCapture, AudioCaptureProtocol)), EPropertyChangeType::ValueSet);
		PostEditChangeProperty(PropertyChangedEvent);
#endif
	}

}

void UMovieSceneCapture::InitializeCaptureProtocols()
{
	UClass* ImageProtocolType = ImageCaptureProtocolType.TryLoadClass<UMovieSceneCaptureProtocolBase>();
	UClass* AudioProtocolType = AudioCaptureProtocolType.TryLoadClass<UMovieSceneCaptureProtocolBase>();

	// If there's no type and we've no protocol, do nothing
	if (!ImageProtocolType && !AudioProtocolType && !ImageCaptureProtocol && !AudioCaptureProtocol)
	{
		return;
	}

	// If we have a type and we've already got a protocol of that type, do nothing
	if (ImageProtocolType && ImageCaptureProtocol && ImageCaptureProtocol->GetClass() == ImageProtocolType && 
		AudioCaptureProtocol && AudioCaptureProtocol->GetClass() == AudioProtocolType)
	{
		return;
	}

	ForciblyReinitializeCaptureProtocols();
}


void UMovieSceneCapture::Initialize(TSharedPtr<FSceneViewport> InSceneViewport, int32 PIEInstance)
{
	ensure(!bCapturing);
	// Apply command-line overrides
	{
		FString OutputPathOverride;
		if( FParse::Value( FCommandLine::Get(), TEXT( "-MovieFolder=" ), OutputPathOverride ) )
		{
			Settings.OutputDirectory.Path = OutputPathOverride;
			FPaths::NormalizeFilename(Settings.OutputDirectory.Path);

			// Only validate the directory if it doesn't contain any format specifiers
			if (!Settings.OutputDirectory.Path.Contains(TEXT("{")))
			{
				if (!IFileManager::Get().DirectoryExists(*Settings.OutputDirectory.Path))
				{
					if (!IFileManager::Get().MakeDirectory(*Settings.OutputDirectory.Path))
					{
						UE_LOGF(LogMovieSceneCapture, Error, "Invalid output directory: %ls.", *Settings.OutputDirectory.Path);
					}
				}
				else if (IFileManager::Get().IsReadOnly(*Settings.OutputDirectory.Path))
				{
					UE_LOGF(LogMovieSceneCapture, Error, "Read only output directory: %ls.", *Settings.OutputDirectory.Path);
				}
			}
		}

		FString OutputNameOverride;
		if( FParse::Value( FCommandLine::Get(), TEXT( "-MovieName=" ), OutputNameOverride ) )
		{
			Settings.OutputFormat = OutputNameOverride;
		}

		bool bOverrideOverwriteExisting;
		if( FParse::Bool( FCommandLine::Get(), TEXT( "-MovieOverwriteExisting=" ), bOverrideOverwriteExisting ) )
		{
			Settings.bOverwriteExisting = bOverrideOverwriteExisting;
		}

		bool bOverrideRelativeFrameNumbers;
		if( FParse::Bool( FCommandLine::Get(), TEXT( "-MovieRelativeFrames=" ), bOverrideRelativeFrameNumbers ) )
		{
			Settings.bUseRelativeFrameNumbers = bOverrideRelativeFrameNumbers;
		}

		int32 HandleFramesOverride;
		if( FParse::Value( FCommandLine::Get(), TEXT( "-HandleFrames=" ), HandleFramesOverride ) )
		{
			Settings.HandleFrames = HandleFramesOverride;
		}

		bool bOverrideCinematicEngineScalabilityMode;
		if( FParse::Bool( FCommandLine::Get(), TEXT( "-MovieEngineScalabilityMode=" ), bOverrideCinematicEngineScalabilityMode ) )
		{
			Settings.bCinematicEngineScalability = bOverrideCinematicEngineScalabilityMode;
		}

		bool bOverrideCinematicMode;
		if( FParse::Bool( FCommandLine::Get(), TEXT( "-MovieCinematicMode=" ), bOverrideCinematicMode ) )
		{
			Settings.bCinematicMode = bOverrideCinematicMode;
		}

		bool bOverridePathTracer;
		if (FParse::Bool(FCommandLine::Get(), TEXT("-PathTracer="), bOverridePathTracer))
		{
			Settings.bUsePathTracer = bOverridePathTracer;
			if (bOverridePathTracer)
			{
				InSceneViewport->GetClient()->GetEngineShowFlags()->SetPathTracing(true);
			}
		}

		uint16 OverridePathTracerSamplePerPixel;
		if (FParse::Value(FCommandLine::Get(), TEXT("-PathTracerSamplePerPixel="), OverridePathTracerSamplePerPixel))
		{
			Settings.PathTracerSamplePerPixel = OverridePathTracerSamplePerPixel;
		}

		bool bProtocolOverride = false;

		FString ImageProtocolOverrideString;
		if (   FParse::Value( FCommandLine::Get(), TEXT( "-MovieFormat=" ), ImageProtocolOverrideString )
			|| FParse::Value( FCommandLine::Get(), TEXT( "-ImageCaptureProtocol=" ), ImageProtocolOverrideString ) )
		{
			static const TCHAR* const CommandLineIDString = TEXT("CommandLineID");

			UClass* OverrideClass = nullptr;

			TArray<UClass*> AllProtocolTypes = FindAllCaptureProtocolClasses();
			for (UClass* Class : AllProtocolTypes)
			{
				bool bMetaDataMatch = 
#if WITH_EDITOR
					Class->GetMetaData(CommandLineIDString) == ImageProtocolOverrideString;
#else
					false;
#endif
				if ( bMetaDataMatch || Class->GetName() == ImageProtocolOverrideString )
				{
					OverrideClass = Class;
					break;
				}
			}

			if (!OverrideClass)
			{
				UE_LOGF(LogMovieSceneCapture, Error, "Unrecognized image capture type (-MovieFormat or -ImageCaptureProtocol): %ls.", *ImageProtocolOverrideString);
			}
			else
			{
				ImageCaptureProtocolType = OverrideClass;				
			}
		}
		
		FString AudioProtocolOverrideString;
		if (FParse::Value( FCommandLine::Get(), TEXT( "-AudioCaptureProtocol=" ), AudioProtocolOverrideString ) )
		{
			static const TCHAR* const CommandLineIDString = TEXT("CommandLineID");

			UClass* OverrideClass = nullptr;

			TArray<UClass*> AllProtocolTypes = FindAllCaptureProtocolClasses();
			for (UClass* Class : AllProtocolTypes)
			{
				bool bMetaDataMatch = 
#if WITH_EDITOR
					Class->GetMetaData(CommandLineIDString) == AudioProtocolOverrideString;
#else
					false;
#endif
				if ( bMetaDataMatch || Class->GetName() == AudioProtocolOverrideString )
				{
					OverrideClass = Class;
					break;
				}
			}

			if (!OverrideClass)
			{
				UE_LOGF(LogMovieSceneCapture, Error, "Unrecognized audio capture type (-AudioCaptureProtocol): %ls.", *AudioProtocolOverrideString);
			}
			else
			{
				AudioCaptureProtocolType = OverrideClass;				
			}
		}
			
		FString FrameRateOverrideString;
		if ( FParse::Value( FCommandLine::Get(), TEXT( "-MovieFrameRate=" ), FrameRateOverrideString ) )
		{
			if (!TryParseString(Settings.CustomFrameRate, *FrameRateOverrideString))
			{
				UE_LOGF(LogMovieSceneCapture, Error, "Unrecognized capture frame rate: %ls. Defaulting to sequence frame rate.", *FrameRateOverrideString);
			}
			else
			{
				Settings.bUseCustomFrameRate = true;
			}
		}
	}

	if (!IsRayTracingEnabled())
	{
		Settings.bUsePathTracer = false;
	}

	bFinalizeWhenReady = false;
	bIsAudioCapturePass = false;

	InitSettings = FCaptureProtocolInitSettings::FromSlateViewport(InSceneViewport.ToSharedRef());

	CachedMetrics = FCachedMetrics();
	CachedMetrics.Width = InitSettings->DesiredSize.X;
	CachedMetrics.Height = InitSettings->DesiredSize.Y;

	double FrameRate = Settings.GetFrameRate().AsDecimal();

	FormatMappings.Reserve(10);
	if (FrameRate == FMath::RoundToDouble(FrameRate))
	{
		FormatMappings.Add(TEXT("fps"), FString::Printf(TEXT("%" INT64_FMT), FMath::RoundToInt(FrameRate)));
	}
	else
	{
		FormatMappings.Add(TEXT("fps"), FString::Printf(TEXT("%.2f"), FrameRate));
	}
	FormatMappings.Add(TEXT("width"), FString::Printf(TEXT("%d"), CachedMetrics.Width));
	FormatMappings.Add(TEXT("height"), FString::Printf(TEXT("%d"), CachedMetrics.Height));
	FormatMappings.Add(TEXT("world"), InSceneViewport->GetClient()->GetWorld()->GetName());

	FDateTime CurrentTime = FDateTime::Now();
	FormatMappings.Add(TEXT("year"), CurrentTime.ToString(TEXT("%Y")));
	FormatMappings.Add(TEXT("month"), CurrentTime.ToString(TEXT("%m")));
	FormatMappings.Add(TEXT("day"), CurrentTime.ToString(TEXT("%d")));
	FormatMappings.Add(TEXT("date"), CurrentTime.ToString(TEXT("%Y.%m.%d")));
	FormatMappings.Add(TEXT("time"), CurrentTime.ToString(TEXT("%H.%M.%S")));

	if( !CaptureStrategy.IsValid() )
	{
		CaptureStrategy = MakeShareable( new FRealTimeCaptureStrategy( Settings.FrameRate ) );
		CaptureStrategy->OnInitialize();
	}

	InitializeCaptureProtocols();
	if (ensure(ImageCaptureProtocol) && ensure(AudioCaptureProtocol))
	{
		ImageCaptureProtocol->Setup(InitSettings.GetValue(), this);
		AudioCaptureProtocol->Setup(InitSettings.GetValue(), this);
	}

	if (Settings.bCinematicEngineScalability)
	{
		CachedQualityLevels = Scalability::GetQualityLevels();
		Scalability::FQualityLevels QualityLevels = CachedQualityLevels;
		QualityLevels.SetFromSingleQualityLevelRelativeToMax(0);
		Scalability::SetQualityLevels(QualityLevels);
	}

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		FActiveMovieSceneCaptures::Get().Add(this);
	}
}

void UMovieSceneCapture::StartWarmup()
{
	bCapturing = false;

	if (bIsAudioCapturePass)
	{
		if (ensure(AudioCaptureProtocol))
		{
			AudioCaptureProtocol->WarmUp();
		}
	}
	else
	{
		if (ensure(ImageCaptureProtocol))
		{
			ImageCaptureProtocol->WarmUp();
		}
	}
}

void UMovieSceneCapture::StartCapture()
{
	bFinalizeWhenReady = false;
	bCapturing = true;

	// Audio Captures will always use Real Time capture strategies due to the Audio engine's need for real-time processing.
	if (!CaptureStrategy.IsValid() || bIsAudioCapturePass)
	{
		if (CaptureStrategy.IsValid())
		{
			CaptureStrategy->OnStop();
		}

		CaptureStrategy = MakeShareable(new FRealTimeCaptureStrategy(Settings.FrameRate));
	}

	CaptureStrategy->OnInitialize();

	// We only initialize the image capture protocol on the first pass and then stop ticking it (but don't finalize it)
	// until the audio capture pass has finished as well. StartCapture can get called up to two times, once for the image
	// pass, and again for the audio pass (if needed).
	if(!bIsAudioCapturePass)
	{
		ImageCaptureProtocol->StartCapture();
		
		// Disable audio so when the image pass runs it doesn't play stuttering audio.
		// ToDo: This doesn't work very well in the editor due to some conflicting code in the engine tick loop
		// that also sets the volume each frame, overriding the effect of this.
		FApp::SetVolumeMultiplier(0.0f);
		FApp::SetUnfocusedVolumeMultiplier(0.0f);
	}
	else
	{
		// Unmute the audio
		FApp::SetVolumeMultiplier(1.0f);
		// Ensure non-focused apps still play audio as the audio has to be emitted for the recording to capture it.
		FApp::SetUnfocusedVolumeMultiplier(1.0f); 
		
		AudioCaptureProtocol->StartCapture();
	}
}

void UMovieSceneCapture::CaptureThisFrame(float DeltaSeconds)
{
	if (!bCapturing || !CaptureStrategy.IsValid() || !ImageCaptureProtocol || !AudioCaptureProtocol || bFinalizeWhenReady)
	{
		return;
	}

	CachedMetrics.ElapsedSeconds += DeltaSeconds;
	if (CaptureStrategy->ShouldPresent(CachedMetrics.ElapsedSeconds, CachedMetrics.Frame))
	{
		uint32 NumDroppedFrames = CaptureStrategy->GetDroppedFrames(CachedMetrics.ElapsedSeconds, CachedMetrics.Frame);
		CachedMetrics.Frame += NumDroppedFrames;

		const FFrameMetrics ThisFrameMetrics(
			CachedMetrics.ElapsedSeconds,
			DeltaSeconds,
			CachedMetrics.Frame,
			NumDroppedFrames
			);
		if (!bIsAudioCapturePass)
		{
			ImageCaptureProtocol->CaptureFrame(ThisFrameMetrics);
		}
		else
		{
			AudioCaptureProtocol->CaptureFrame(ThisFrameMetrics);
		}
		UE_LOGF(LogMovieSceneCapture, Verbose, "Captured frame: %d", CachedMetrics.Frame);
		++CachedMetrics.Frame;
	}
}

void UMovieSceneCapture::Tick(float DeltaSeconds)
{
	if (ImageCaptureProtocol)
	{
		ImageCaptureProtocol->PreTick();
	}
	if(AudioCaptureProtocol)
	{
		AudioCaptureProtocol->PreTick();
	}

	OnTick(DeltaSeconds);

	if (ImageCaptureProtocol)
	{
		ImageCaptureProtocol->Tick();
	}
	
	if(AudioCaptureProtocol)
	{
		AudioCaptureProtocol->Tick();
	}
}

void UMovieSceneCapture::FinalizeWhenReady()
{
	if (!bFinalizeWhenReady)
	{
		bFinalizeWhenReady = true;

		if (ImageCaptureProtocol)
		{
			ImageCaptureProtocol->BeginFinalize();
		}
		
		if(AudioCaptureProtocol)
		{
			AudioCaptureProtocol->BeginFinalize();
		}
	}
}

void UMovieSceneCapture::Finalize()
{
	if (Settings.bCinematicEngineScalability)
	{
		Scalability::SetQualityLevels(CachedQualityLevels);
	}

	FActiveMovieSceneCaptures::Get().Remove(this);

	if (bCapturing)
	{
		bCapturing = false;

		if (CaptureStrategy.IsValid())
		{
			CaptureStrategy->OnStop();
			CaptureStrategy = nullptr;
		}

		if (ImageCaptureProtocol)
		{
			ImageCaptureProtocol->Finalize();
		}
		if(AudioCaptureProtocol)
		{
			AudioCaptureProtocol->Finalize();
		}
		
		// Reinitialize the object to ensure no transient state is carried over from one capture to the next
		ForciblyReinitializeCaptureProtocols();

		OnCaptureFinishedDelegate.Broadcast();
	}

	bFinalizeWhenReady = false;
}

FString UMovieSceneCapture::ResolveFileFormat(const FString& Format, const FFrameMetrics& FrameMetrics) const
{
	TMap<FString, FStringFormatArg> AllArgs = FormatMappings;

	AllArgs.Add(TEXT("frame"), FString::Printf(TEXT("%0*d"), Settings.ZeroPadFrameNumbers, Settings.bUseRelativeFrameNumbers ? FrameMetrics.FrameNumber : FrameMetrics.FrameNumber + FrameNumberOffset));

	AddFormatMappings(AllArgs, FrameMetrics);

	// Allow the capture protocols to add their own format mappings
	if (ImageCaptureProtocol)
	{
		ImageCaptureProtocol->AddFormatMappings(AllArgs);
	}
	if (AudioCaptureProtocol)
	{
		AudioCaptureProtocol->AddFormatMappings(AllArgs);
	}

	return FString::Format(*Format, AllArgs);
}

void UMovieSceneCapture::LoadFromConfig()
{
	LoadConfig();
	InitializeCaptureProtocols();
	if(ImageCaptureProtocol)
	{
		ImageCaptureProtocol->LoadConfig();
	}
	if(AudioCaptureProtocol)
	{
		AudioCaptureProtocol->LoadConfig();
	}

	FString JsonString;
	FString Section = GetClass()->GetPathName() + TEXT("_Json");

	if (GConfig->GetString( *Section, TEXT("Data"), JsonString, GEditorSettingsIni))
	{
		FString UnescapedString = FRemoteConfig::ReplaceIniSpecialCharWithChar(JsonString).ReplaceEscapedCharWithChar();

		TSharedPtr<FJsonObject> RootObject;
		TSharedRef<TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(UnescapedString);
		if (FJsonSerializer::Deserialize(JsonReader, RootObject) && RootObject.IsValid())
		{
			DeserializeAdditionalJson(*RootObject);
		}
	}
}

void UMovieSceneCapture::SaveToConfig()
{
	TSharedRef<FJsonObject> Json = MakeShareable(new FJsonObject);
	SerializeAdditionalJson(*Json);

	FString JsonString;
	TSharedRef<TJsonWriter<> > JsonWriter = TJsonWriterFactory<>::Create(&JsonString, 0);
	if (FJsonSerializer::Serialize( Json, JsonWriter ))
	{
		FString Section = GetClass()->GetPathName() + TEXT("_Json");

		FString EscapedJson = FRemoteConfig::ReplaceIniCharWithSpecialChar(JsonString).ReplaceCharWithEscapedChar();

		GConfig->SetString( *Section, TEXT("Data"), *EscapedJson, GEditorSettingsIni);
		GConfig->Flush(false, GEditorSettingsIni);
	}

	SaveConfig();
	
	if(ImageCaptureProtocol)
	{
		ImageCaptureProtocol->SaveConfig();
	}
	if(AudioCaptureProtocol)
	{
		AudioCaptureProtocol->SaveConfig();
	}
}
void UMovieSceneCapture::SerializeJson(FJsonObject& Object)
{
	if (ImageCaptureProtocol)
	{
		Object.SetField(TEXT("ImageProtocolType"), MakeShareable(new FJsonValueString(ImageCaptureProtocol->GetClass()->GetPathName())));
		TSharedRef<FJsonObject> ProtocolDataObject = MakeShareable(new FJsonObject);
		if (FJsonObjectConverter::UStructToJsonObject(ImageCaptureProtocol->GetClass(), ImageCaptureProtocol, ProtocolDataObject, 0, 0))
		{
			Object.SetField(TEXT("ImageProtocolData"), MakeShareable(new FJsonValueObject(ProtocolDataObject)));
		}
	}
	
	if (AudioCaptureProtocol)
	{
		Object.SetField(TEXT("AudioProtocolType"), MakeShareable(new FJsonValueString(AudioCaptureProtocol->GetClass()->GetPathName())));
		TSharedRef<FJsonObject> ProtocolDataObject = MakeShareable(new FJsonObject);
		if (FJsonObjectConverter::UStructToJsonObject(AudioCaptureProtocol->GetClass(), AudioCaptureProtocol, ProtocolDataObject, 0, 0))
		{
			Object.SetField(TEXT("AudioProtocolData"), MakeShareable(new FJsonValueObject(ProtocolDataObject)));
		}
	}

	SerializeAdditionalJson(Object);
}

void UMovieSceneCapture::DeserializeJson(const FJsonObject& Object)
{
	TSharedPtr<FJsonValue> ImageProtocolTypeField = Object.TryGetField(TEXT("ImageProtocolType"));
	if (ImageProtocolTypeField.IsValid())
	{
		UClass* ProtocolTypeClass = FindObject<UClass>(nullptr, *ImageProtocolTypeField->AsString());
		if (ProtocolTypeClass && ProtocolTypeClass->IsChildOf(UMovieSceneCaptureProtocolBase::StaticClass()))
		{
			SetImageCaptureProtocolType(ProtocolTypeClass);
			if (ImageCaptureProtocol)
			{
				TSharedPtr<FJsonValue> ProtocolDataField = Object.TryGetField(TEXT("ImageProtocolData"));
				if (ProtocolDataField.IsValid())
				{
					FJsonObjectConverter::JsonAttributesToUStruct(ProtocolDataField->AsObject()->Values, ProtocolTypeClass, ImageCaptureProtocol, 0, 0);
				}
			}

#if WITH_EDITOR
			FPropertyChangedEvent PropertyChangedEvent(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMovieSceneCapture, ImageCaptureProtocol)), EPropertyChangeType::ValueSet);
			PostEditChangeProperty(PropertyChangedEvent);
#endif
		}
	}
	
	TSharedPtr<FJsonValue> AudioProtocolTypeField = Object.TryGetField(TEXT("AudioProtocolType"));
	if (AudioProtocolTypeField.IsValid())
	{
		UClass* ProtocolTypeClass = FindObject<UClass>(nullptr, *AudioProtocolTypeField->AsString());
		if (ProtocolTypeClass && ProtocolTypeClass->IsChildOf(UMovieSceneCaptureProtocolBase::StaticClass()))
		{
			SetAudioCaptureProtocolType(ProtocolTypeClass);
			if (AudioCaptureProtocol)
			{
				TSharedPtr<FJsonValue> ProtocolDataField = Object.TryGetField(TEXT("AudioProtocolData"));
				if (ProtocolDataField.IsValid())
				{
					FJsonObjectConverter::JsonAttributesToUStruct(ProtocolDataField->AsObject()->Values, ProtocolTypeClass, AudioCaptureProtocol, 0, 0);
				}
			}

#if WITH_EDITOR
			FPropertyChangedEvent PropertyChangedEvent(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMovieSceneCapture, AudioCaptureProtocol)), EPropertyChangeType::ValueSet);
			PostEditChangeProperty(PropertyChangedEvent);
#endif
		}
	}

	DeserializeAdditionalJson(Object);
}


bool UMovieSceneCapture::ShouldFinalize() const
{
	return bFinalizeWhenReady && ImageCaptureProtocol->HasFinishedProcessing() && AudioCaptureProtocol->HasFinishedProcessing();
}

bool UMovieSceneCapture::IsAudioPassIfNeeded() const
{
	return AudioCaptureProtocolType == UNullAudioCaptureProtocol::StaticClass() || bIsAudioCapturePass;
}

#if WITH_EDITOR
void UMovieSceneCapture::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FName PropertyName = (PropertyChangedEvent.MemberProperty != NULL) ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UMovieSceneCapture, ImageCaptureProtocolType)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UMovieSceneCapture, AudioCaptureProtocolType))
	{
		InitializeCaptureProtocols();
	}

	// We only want to save changes to the UI instance. This makes it so that closing the Movie Scene Capture UI
	// saves your changes (without having to render a movie) but doesn't leak changes into the Python scripting
	// environment.
	if (GetFName() == MovieSceneCaptureUIName)
	{
		SaveToConfig();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

FFixedTimeStepCaptureStrategy::FFixedTimeStepCaptureStrategy(FFrameRate InFrameRate)
	: FrameRate(InFrameRate)
{
}

void FFixedTimeStepCaptureStrategy::OnInitialize()
{
	FApp::SetFixedDeltaTime(FrameRate.AsInterval());
	FApp::SetUseFixedTimeStep(true);
}

void FFixedTimeStepCaptureStrategy::OnStop()
{
	FApp::SetUseFixedTimeStep(false);

	// This can cause an issue with negative delta times in the engine. Instead of changing
	// that rather sensitive code, we're going to just override the time that calculation is based off of, so it never comes up with a negative number.
	if(FApp::GetCurrentTime() > FPlatformTime::Seconds())
	{
		UE_LOGF(LogTemp, Warning, "FApp::CurrentTime() is ahead of platform time due to Fixed Timestep. Fixing. AppCurrentTime: %f PlatformCurrentTime: %f", FApp::GetCurrentTime(), FPlatformTime::Seconds());
		double CurrentRealTime = FPlatformTime::Seconds();
		FApp::SetCurrentTime(CurrentRealTime);
	}
}

bool FFixedTimeStepCaptureStrategy::ShouldPresent(double CurrentTimeSeconds, uint32 FrameIndex) const
{
	return true;
}

int32 FFixedTimeStepCaptureStrategy::GetDroppedFrames(double CurrentTimeSeconds, uint32 FrameIndex) const
{
	return 0;
}

FRealTimeCaptureStrategy::FRealTimeCaptureStrategy(FFrameRate InFrameRate)
	: NextPresentTimeS(0), FrameLength(InFrameRate.AsInterval())
{
}

void FRealTimeCaptureStrategy::OnInitialize()
{
}

void FRealTimeCaptureStrategy::OnStop()
{
}

bool FRealTimeCaptureStrategy::ShouldPresent(double CurrentTimeSeconds, uint32 FrameIndex) const
{
	return CurrentTimeSeconds >= FrameIndex * FrameLength;
}

int32 FRealTimeCaptureStrategy::GetDroppedFrames(double CurrentTimeSeconds, uint32 FrameIndex) const
{
	uint32 ThisFrame = FMath::FloorToInt(CurrentTimeSeconds / FrameLength);
	if (ThisFrame > FrameIndex)
	{
		return ThisFrame - FrameIndex;
	}
	return 0;
}

#undef LOCTEXT_NAMESPACE

