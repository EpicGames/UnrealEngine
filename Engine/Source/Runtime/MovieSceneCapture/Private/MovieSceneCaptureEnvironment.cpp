// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneCaptureEnvironment.h"
#include "MovieSceneCapture.h"
#include "MovieSceneCaptureModule.h"
#include "Protocols/UserDefinedCaptureProtocol.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCaptureEnvironment)

int32 UMovieSceneCaptureEnvironment::GetCaptureFrameNumber()
{
	UMovieSceneCapture* Capture = static_cast<UMovieSceneCapture*>(IMovieSceneCaptureModule::Get().GetFirstActiveMovieSceneCapture());
	return Capture ? Capture->GetMetrics().Frame : 0;
}

float UMovieSceneCaptureEnvironment::GetCaptureElapsedTime()
{
	UMovieSceneCapture* Capture = static_cast<UMovieSceneCapture*>(IMovieSceneCaptureModule::Get().GetFirstActiveMovieSceneCapture());
	return Capture ? Capture->GetMetrics().ElapsedSeconds : 0.f;
}

bool UMovieSceneCaptureEnvironment::IsCaptureInProgress()
{
	return IMovieSceneCaptureModule::Get().GetFirstActiveMovieSceneCapture() != nullptr;
}

UMovieSceneImageCaptureProtocolBase* UMovieSceneCaptureEnvironment::FindImageCaptureProtocol()
{
	UMovieSceneCapture* Capture = static_cast<UMovieSceneCapture*>(IMovieSceneCaptureModule::Get().GetFirstActiveMovieSceneCapture());
	return Capture ? Cast<UMovieSceneImageCaptureProtocolBase>(Capture->GetImageCaptureProtocol()) : nullptr;
}

UMovieSceneAudioCaptureProtocolBase* UMovieSceneCaptureEnvironment::FindAudioCaptureProtocol()
{
	UMovieSceneCapture* Capture = static_cast<UMovieSceneCapture*>(IMovieSceneCaptureModule::Get().GetFirstActiveMovieSceneCapture());
	return Capture ? Cast<UMovieSceneAudioCaptureProtocolBase>(Capture->GetAudioCaptureProtocol()) : nullptr;
}
