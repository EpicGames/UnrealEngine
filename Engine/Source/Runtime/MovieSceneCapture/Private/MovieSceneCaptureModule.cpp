// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneCaptureModule.h"
#include "Misc/CoreDelegates.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Class.h"
#include "UObject/CoreRedirects.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "EngineGlobals.h"
#include "Engine/GameEngine.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "UObject/Package.h"
#include "MovieSceneCapture.h"
#include "JsonObjectConverter.h"
#include "ActiveMovieSceneCaptures.h"
#include "Slate/SceneViewport.h"
#include "ViewportClient.h"

#define LOCTEXT_NAMESPACE "MovieSceneCapture"

DEFINE_LOG_CATEGORY(LogMovieSceneCapture);

class FMovieSceneCaptureModule : public IMovieSceneCaptureModule
{
private:

	/** Handle to a movie capture implementation created from the command line, to be initialized once a world is loaded */
	FMovieSceneCaptureHandle StartupMovieCaptureHandle;
	bool bStereoAllowed;

	virtual bool IsStereoAllowed() override
	{
		return bStereoAllowed;
	}
	
	virtual void StartupModule() override
	{
		bStereoAllowed = false;

		FCoreDelegates::OnPreExit.AddRaw(this, &FMovieSceneCaptureModule::PreExit);
		FCoreUObjectDelegates::PostLoadMapWithWorld.AddRaw(this, &FMovieSceneCaptureModule::OnPostLoadMap );
	}

	void PreExit()
	{
		FActiveMovieSceneCaptures::Get().Shutdown();
	}

	void OnPostLoadMap(UWorld*)
	{
		UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);
		if (GameEngine && StartupMovieCaptureHandle.IsValid())
		{
			if (!GameEngine->SceneViewport->GetClient()->GetWorld())
			{
				// @todo: Set exit code to EMovieSceneCaptureExitCode::WorldNotFound when we have the ability to do so
				FPlatformMisc::RequestExit(false, TEXT("FMovieSceneCaptureModule::OnPostLoadMap"));
			}
			else
			{
				IMovieSceneCaptureInterface* StartupCaptureInterface = RetrieveMovieSceneInterface(StartupMovieCaptureHandle);
				StartupCaptureInterface->Initialize(GameEngine->SceneViewport.ToSharedRef());
			}
		}

		StartupMovieCaptureHandle = FMovieSceneCaptureHandle();
		FCoreUObjectDelegates::PostLoadMapWithWorld.RemoveAll(this);
	}

	virtual void PreUnloadCallback() override
	{
		DestroyAllActiveCaptures();
	}

	virtual IMovieSceneCaptureInterface* InitializeFromCommandLine() override
	{
		if (GIsEditor)
		{
			return nullptr;
		}
		
		if (FParse::Param(FCommandLine::Get(), TEXT("EmulateStereo")))
		{
			bStereoAllowed = true;
		}
		
		FString TypeName;
		if( FParse::Value( FCommandLine::Get(), TEXT( "-MovieSceneCaptureType=" ), TypeName ) )
		{
			// OK, they specified the type of capture they want on the command-line.  Manifests are now optional!
		}

		FString ManifestPath;
		if (!FParse::Value(FCommandLine::Get(), TEXT("-MovieSceneCaptureManifest="), ManifestPath) || ManifestPath.IsEmpty())
		{
			// Allow capturing without a manifest.  Command-line parameters for individual options will be necessary.
			if( TypeName.IsEmpty() )
			{
				return nullptr;
			}
		}

		UMovieSceneCapture* Capture = nullptr;
		if( !ManifestPath.IsEmpty() )
		{
			FString Json;
			if (FFileHelper::LoadFileToString(Json, *ManifestPath))
			{
				TSharedPtr<FJsonObject> RootObject;
				TSharedRef<TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(Json);
				if (FJsonSerializer::Deserialize(JsonReader, RootObject) && RootObject.IsValid())
				{
					auto TypeField = RootObject->TryGetField(TEXT("Type"));
					if (!TypeField.IsValid())
					{
						return nullptr;
					}

					TypeName = TypeField->AsString();
					UClass* Class = FindObject<UClass>(nullptr, *TypeName);
					if (!Class)
					{
						const FCoreRedirect* ValueRedirect = nullptr;
						FCoreRedirectObjectName NewRedirectName;

						if (FCoreRedirects::RedirectNameAndValues(ECoreRedirectFlags::Type_Class, TypeName, NewRedirectName, &ValueRedirect))
						{
							Class = FindObject<UClass>(nullptr, *NewRedirectName.ToString());
						}

						if (!Class)
						{
							return nullptr;
						}
					}

					Capture = NewObject<UMovieSceneCapture>(GetTransientPackage(), Class);
					if (!Capture)
					{
						return nullptr;
					}

					auto DataField = RootObject->TryGetField(TEXT("Data"));
					if (!DataField.IsValid())
					{
						return nullptr;
					}

					if (!FJsonObjectConverter::JsonAttributesToUStruct(DataField->AsObject()->Values, Class, Capture, 0, 0))
					{
						return nullptr;
					}

					TSharedPtr<FJsonValue> AdditionalDataField = RootObject->TryGetField(TEXT("AdditionalData"));
					if (AdditionalDataField.IsValid())
					{
						Capture->DeserializeJson(*AdditionalDataField->AsObject());
					}
				}
			}
		}
		else if( !TypeName.IsEmpty() )
		{
			UClass* Class = FindObject<UClass>( nullptr, *TypeName );
			if( !Class )
			{
				const FCoreRedirect* ValueRedirect = nullptr;
				FCoreRedirectObjectName NewRedirectName;

				if (FCoreRedirects::RedirectNameAndValues(ECoreRedirectFlags::Type_Class, TypeName, NewRedirectName, &ValueRedirect))
				{
					Class = FindObject<UClass>(nullptr, *NewRedirectName.ToString());
				}

				if (!Class)
				{
					return nullptr;
				}
			}

			Capture = NewObject<UMovieSceneCapture>( GetTransientPackage(), Class );
			if( !Capture )
			{
				return nullptr;
			}
		}

		check( Capture != nullptr );
		StartupMovieCaptureHandle = Capture->GetHandle();
		// Add it immediately, so we can get back to it from its handle (usually it gets added in Initialize())
		FActiveMovieSceneCaptures::Get().Add( Capture );

		Capture->OnCaptureFinished().AddLambda([]{
			FPlatformMisc::RequestExit(false, TEXT("FMovieSceneCaptureModule::InitializeFromCommandLine"));
		});
		
		return Capture;
	}

	virtual IMovieSceneCaptureInterface* CreateMovieSceneCapture(TSharedPtr<FSceneViewport> InSceneViewport) override
	{
		UMovieSceneCapture* Capture = NewObject<UMovieSceneCapture>(GetTransientPackage());
		Capture->Initialize(InSceneViewport);
		Capture->StartCapture();
		return Capture;
	}

	virtual IMovieSceneCaptureInterface* RetrieveMovieSceneInterface(FMovieSceneCaptureHandle Handle)
	{
		for (auto* Existing : FActiveMovieSceneCaptures::Get().GetActiveCaptures())
		{
			if (Existing->GetHandle() == Handle)
			{
				return Existing;
			}
		}
		return nullptr;
	}

	IMovieSceneCaptureInterface* GetFirstActiveMovieSceneCapture()
	{
		if (const TArray<UMovieSceneCapture*>& ActiveCaptures = FActiveMovieSceneCaptures::Get().GetActiveCaptures(); ActiveCaptures.Num() > 0)
		{
			return ActiveCaptures[0];
		}

		return nullptr;
	}

	virtual void DestroyMovieSceneCapture(FMovieSceneCaptureHandle Handle)
	{
		for (auto* Existing : FActiveMovieSceneCaptures::Get().GetActiveCaptures())
		{
			if (Existing->GetHandle() == Handle)
			{
				Existing->Close();
				break;
			}
		}
	}

	virtual void DestroyAllActiveCaptures()
	{
		FCoreDelegates::OnPreExit.RemoveAll(this);
		PreExit();
	}

};

IMPLEMENT_MODULE( FMovieSceneCaptureModule, MovieSceneCapture )

#undef LOCTEXT_NAMESPACE
