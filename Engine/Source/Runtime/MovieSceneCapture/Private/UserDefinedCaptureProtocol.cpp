// Copyright Epic Games, Inc. All Rights Reserved.

#include "Protocols/UserDefinedCaptureProtocol.h"
#include "ImageWriteQueue.h"
#include "ImageWriteBlueprintLibrary.h"
#include "Modules/ModuleManager.h"
#include "Async/Async.h"
#include "Engine/Texture.h"
#include "UnrealClient.h"
#include "MovieSceneCaptureModule.h"
#include "MovieSceneCaptureSettings.h"
#include "Slate/SceneViewport.h"
#include "ImagePixelData.h"
#include "Engine/Engine.h"
#include "Logging/MessageLog.h"
#include "ViewportClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UserDefinedCaptureProtocol)

#define LOCTEXT_NAMESPACE "UserDefinedImageCaptureProtocol"

struct FCaptureProtocolFrameData : IFramePayload
{
	FFrameMetrics Metrics;

	FCaptureProtocolFrameData(const FFrameMetrics& InMetrics)
		: Metrics(InMetrics)
	{}
};

/** Callable utility struct that calls a handler with the specified parameters on the game thread */
struct FCallPixelHandler_GameThread
{
	static void Dispatch(const FCapturedPixelsID& InStreamID, const FFrameMetrics& InFrameMetrics, const FCapturedPixels& InPixels, TWeakObjectPtr<UUserDefinedCaptureProtocol> InWeakProtocol)
	{
		FCallPixelHandler_GameThread Functor;
		Functor.Pixels           = InPixels;
		Functor.FrameMetrics     = InFrameMetrics;
		Functor.StreamID         = InStreamID;
		Functor.WeakProtocol     = InWeakProtocol;

		if (!IsInGameThread())
		{
			AsyncTask(ENamedThreads::GameThread, MoveTemp(Functor));
		}
		else
		{
			Functor();
		}
	}

	void operator()()
	{
		check(IsInGameThread());

		UUserDefinedCaptureProtocol* Protocol = WeakProtocol.Get();
		if (Protocol)
		{
			Protocol->OnPixelsReceivedImpl(Pixels, StreamID, FrameMetrics);
		}
	}

private:

	/** The captured pixels themselves */
	FCapturedPixels Pixels;
	/** The ID of the stream that these pixels represent */
	FCapturedPixelsID StreamID;
	/** Metrics for the frame from which the pixel data is derived */
	FFrameMetrics FrameMetrics;
	/** Weak pointer back to the protocol. Only used to invoke OnBufferReady. */
	TWeakObjectPtr<UUserDefinedCaptureProtocol> WeakProtocol;
};

FString FCapturedPixelsID::ToString() const
{
	FString Name;
	for (const TTuple<FName, FName>& Pair : Identifiers)
	{
		if (Name.Len() > 0)
		{
			Name += TEXT(",");
		}

		Name += Pair.Key.ToString();
		if (Pair.Value != NAME_None)
		{
			Name += TEXT(":");
			Name += Pair.Value.ToString();
		}
	}
	return Name.Len() > 0 ? Name : TEXT("<none>");
}


UUserDefinedCaptureProtocol::UUserDefinedCaptureProtocol(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, World(nullptr)
	, NumOutstandingOperations(0)
{
	CurrentStreamID = nullptr;
}

void UUserDefinedCaptureProtocol::PreTickImpl()
{
	OnPreTick();
}

void UUserDefinedCaptureProtocol::TickImpl()
{
	OnTick();

	// Process any frames that have been captured from the frame grabber
	if (FinalPixelsFrameGrabber.IsValid())
	{
		TArray<FCapturedFrameData> CapturedFrames = FinalPixelsFrameGrabber->GetCapturedFrames();

		for (FCapturedFrameData& Frame : CapturedFrames)
		{
			// Steal the frame and make it shareable
			FCapturedPixels CapturedPixels { MakeShared<TImagePixelData<FColor>, ESPMode::ThreadSafe>( Frame.BufferSize, TArray64<FColor>(MoveTemp(Frame.ColorBuffer) ) ) };
			FFrameMetrics   CapturedMetrics = static_cast<FCaptureProtocolFrameData*>(Frame.Payload.Get())->Metrics;

			// Call the handler
			OnPixelsReceivedImpl(CapturedPixels, FinalPixelsID, CapturedMetrics);
		}
	}
}

bool UUserDefinedCaptureProtocol::SetupImpl()
{
	if (FViewportClient* Client = InitSettings->SceneViewport->GetClient())
	{
		World = Client->GetWorld();
	}
	else
	{
		World = nullptr;
	}

	int32 PreviousPlayInEditorID = UE::GetPlayInEditorID();

	if (World)
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (World == Context.World())
			{
				UE::SetPlayInEditorID(Context.PIEInstance);
			}
		}
	}

	// Preemptively create the frame grabber for final pixels, but do not start capturing final pixels until instructed
	FinalPixelsFrameGrabber.Reset(new FFrameGrabber(InitSettings->SceneViewport.ToSharedRef(), InitSettings->DesiredSize, PF_B8G8R8A8, 3));
	const bool bSuccess = OnSetup();

	UE::SetPlayInEditorID(PreviousPlayInEditorID);

	return bSuccess;
}

void UUserDefinedCaptureProtocol::WarmUpImpl()
{
	OnWarmUp();
}

bool UUserDefinedCaptureProtocol::StartCaptureImpl()
{
	OnStartCapture();
	return true;
}

void UUserDefinedCaptureProtocol::StartCapturingFinalPixels(const FCapturedPixelsID& StreamID)
{
	if (FinalPixelsFrameGrabber.IsValid() && GetState() == EMovieSceneCaptureProtocolState::Capturing && !FinalPixelsFrameGrabber->IsCapturingFrames())
	{
		FinalPixelsID = StreamID;
		FinalPixelsFrameGrabber->StartCapturingFrames();
	}
}

void UUserDefinedCaptureProtocol::StopCapturingFinalPixels()
{
	if (FinalPixelsFrameGrabber.IsValid() && GetState() == EMovieSceneCaptureProtocolState::Capturing && FinalPixelsFrameGrabber->IsCapturingFrames())
	{
		FinalPixelsFrameGrabber->StopCapturingFrames();
	}
}

void UUserDefinedCaptureProtocol::BeginFinalizeImpl()
{
	StopCapturingFinalPixels();

	OnBeginFinalize();
}

bool UUserDefinedCaptureProtocol::HasFinishedProcessingImpl() const
{
	if (NumOutstandingOperations != 0)
	{
		return false;
	}

	// If the frame grabber is still processing, we still have work to do
	if (FinalPixelsFrameGrabber.IsValid() && FinalPixelsFrameGrabber->HasOutstandingFrames())
	{
		return false;
	}

	return OnCanFinalize();
}

void UUserDefinedCaptureProtocol::FinalizeImpl()
{
	if (FinalPixelsFrameGrabber.IsValid())
	{
		FinalPixelsFrameGrabber->Shutdown();
		FinalPixelsFrameGrabber.Reset();
	}

	OnFinalize();
}

void UUserDefinedCaptureProtocol::CaptureFrameImpl(const FFrameMetrics& InFrameMetrics)
{
	CachedFrameMetrics = InFrameMetrics;

	if (FinalPixelsFrameGrabber.IsValid() && FinalPixelsFrameGrabber->IsCapturingFrames())
	{
		ReportOutstandingWork(1);
		FinalPixelsFrameGrabber->CaptureThisFrame(MakeShared<FCaptureProtocolFrameData, ESPMode::ThreadSafe>(CachedFrameMetrics));
	}

	OnCaptureFrame();
}

void UUserDefinedCaptureProtocol::ResolveBuffer(UTexture* Buffer, const FCapturedPixelsID& StreamID)
{
	if (!IsCapturing())
	{
		FFrame::KismetExecutionMessage(TEXT("Capture protocol is not currently capturing frames."), ELogVerbosity::Error);
		return;
	}

	TWeakObjectPtr<UUserDefinedCaptureProtocol> WeakProtocol = MakeWeakObjectPtr(this);
	FFrameMetrics FrameMetrics = CachedFrameMetrics;

	// Capture the current state by-value into the lambda so it can be correctly processed by the thread that resolves the pixels
	auto OnPixelsReady = [StreamID, FrameMetrics, WeakProtocol](TUniquePtr<FImagePixelData>&& PixelData)
	{
		FCapturedPixels CapturedPixels = { MakeShareable(PixelData.Release()) };
		FCallPixelHandler_GameThread::Dispatch(StreamID, FrameMetrics, CapturedPixels, WeakProtocol);
	};

	// Resolve the texture data
	if (UImageWriteBlueprintLibrary::ResolvePixelData(Buffer, OnPixelsReady))
	{
		ReportOutstandingWork(1);
	}
}

void UUserDefinedCaptureProtocol::OnPixelsReceivedImpl(const FCapturedPixels& Pixels, const FCapturedPixelsID& StreamID, FFrameMetrics FrameMetrics)
{
	--NumOutstandingOperations;
	if (Pixels.ImageData->IsDataWellFormed())
	{
		OnPixelsReceived(Pixels, StreamID, FrameMetrics);
	}
}

FString UUserDefinedCaptureProtocol::GenerateFilename(const FFrameMetrics& InFrameMetrics) const
{
	if (!CaptureHost)
	{
		FFrame::KismetExecutionMessage(TEXT("Capture protocol is not currently set up to generate filenames."), ELogVerbosity::Error);
		return FString();
	}

	FString Filename = Super::GenerateFilenameImpl(InFrameMetrics, TEXT(""));
	EnsureFileWritableImpl(Filename);
	return Filename;
}

void UUserDefinedCaptureProtocol::AddFormatMappingsImpl(TMap<FString, FStringFormatArg>& FormatMappings) const
{
	if (CurrentStreamID)
	{
		for (const TTuple<FName, FName>& Pair : CurrentStreamID->Identifiers)
		{
			if (Pair.Value == NAME_None)
			{
				FormatMappings.Add(Pair.Key.ToString(), FString());
			}
			else
			{
				FormatMappings.Add(Pair.Key.ToString(), Pair.Value.ToString());
			}
		}
	}
}

void UUserDefinedCaptureProtocol::ReportOutstandingWork(int32 NumNewOperations)
{
	NumOutstandingOperations += NumNewOperations;
}

UUserDefinedImageCaptureProtocol::UUserDefinedImageCaptureProtocol(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, Format(EDesiredImageFormat::EXR)
	, bEnableCompression(false)
	, CompressionQuality(100)
{}

void UUserDefinedImageCaptureProtocol::OnReleaseConfigImpl(FMovieSceneCaptureSettings& InSettings)
{
	// Remove .{frame} if it exists
	InSettings.OutputFormat = InSettings.OutputFormat.Replace(TEXT(".{frame}"), TEXT(""));
}

void UUserDefinedImageCaptureProtocol::OnLoadConfigImpl(FMovieSceneCaptureSettings& InSettings)
{
	FString OutputFormat = InSettings.OutputFormat;

	// Ensure the format string tries to always export a uniquely named frame so the file doesn't overwrite itself if the user doesn't add it.
	bool bHasFrameFormat = OutputFormat.Contains(TEXT("{frame}")) || OutputFormat.Contains(TEXT("{shot_frame}"));
	if (!bHasFrameFormat)
	{
		OutputFormat.Append(TEXT(".{frame}"));

		InSettings.OutputFormat = OutputFormat;
		UE_LOGF(LogMovieSceneCapture, Display, "Automatically appended .{frame} to the format string as specified format string did not provide a way to differentiate between frames via {frame} or {shot_frame}!");
	}
}

FString UUserDefinedImageCaptureProtocol::GenerateFilename(const FFrameMetrics& InFrameMetrics) const
{
	if (!CaptureHost)
	{
		FFrame::KismetExecutionMessage(TEXT("Capture protocol is not currently set up to generate filenames."), ELogVerbosity::Error);
		return FString();
	}

	const TCHAR* Extension = TEXT("");
	switch (Format)
	{
	case EDesiredImageFormat::BMP: Extension = TEXT(".bmp"); break;
	case EDesiredImageFormat::PNG: Extension = TEXT(".png"); break;
	case EDesiredImageFormat::JPG: Extension = TEXT(".jpg"); break;

	case EDesiredImageFormat::EXR:
	default:
		Extension = TEXT(".exr");
		break;
	}

	FString Filename = GenerateFilenameImpl(InFrameMetrics, Extension);
	EnsureFileWritableImpl(Filename);
	return Filename;
}

FString UUserDefinedImageCaptureProtocol::GenerateFilenameForCurrentFrame()
{
	return GenerateFilename(CachedFrameMetrics);
}

FString UUserDefinedImageCaptureProtocol::GenerateFilenameForBuffer(UTexture* Buffer, const FCapturedPixelsID& StreamID)
{
	if (!CaptureHost)
	{
		FFrame::KismetExecutionMessage(TEXT("Capture protocol is not currently set up to generate filenames."), ELogVerbosity::Error);
		return FString();
	}

	const TCHAR* Extension = TEXT(".ext");
	switch (Format)
	{
	case EDesiredImageFormat::EXR: Extension = TEXT(".exr"); break;
	case EDesiredImageFormat::BMP: Extension = TEXT(".bmp"); break;
	case EDesiredImageFormat::PNG: Extension = TEXT(".png"); break;
	case EDesiredImageFormat::JPG: Extension = TEXT(".jpg"); break;
	default: break;
	}

	CurrentStreamID = &StreamID;

	FString Filename = GenerateFilenameImpl(CachedFrameMetrics, Extension);
	EnsureFileWritableImpl(Filename);

	CurrentStreamID = nullptr;

	return Filename;
}

void UUserDefinedImageCaptureProtocol::WriteImageToDisk(const FCapturedPixels& PixelData, const FCapturedPixelsID& StreamID, const FFrameMetrics& FrameMetrics, bool bCopyImageData)
{
	if (!PixelData.ImageData.IsValid())
	{
		return;
	}
	else if (PixelData.ImageData->GetBitDepth() != 8)
	{
		if (Format == EDesiredImageFormat::BMP)
		{
			FMessageLog("PIE")
			.Warning(FText::Format(LOCTEXT("InvalidBMPExport", "Unable to write the specified render target (stream '{0}' is {1}bit) as BMP. BMPs must be supplied 8bit render targets."),
				FText::FromString(StreamID.ToString()), FText::AsNumber(PixelData.ImageData->GetBitDepth())));
			return;
		}
		else if (Format == EDesiredImageFormat::JPG)
		{
			FMessageLog("PIE")
				.Warning(FText::Format(LOCTEXT("InvalidJPGExport", "Unable to write the specified render target (stream '{0}' is {1}bit) as JPG. JPGs must be supplied 8bit render targets."),
					FText::FromString(StreamID.ToString()), FText::AsNumber(PixelData.ImageData->GetBitDepth())));
			return;
		}
	}

	TUniquePtr<FImageWriteTask> ImageTask = MakeUnique<FImageWriteTask>();

	// Cache the buffer ID so we generate the correct filename
	CurrentStreamID = &StreamID;

	ImageTask->PixelData = bCopyImageData ? PixelData.ImageData->CopyImageData() : PixelData.ImageData->MoveImageDataToNew();
	ImageTask->Filename = GenerateFilename(FrameMetrics);
	ImageTask->Format = ImageFormatFromDesired(Format);
	ImageTask->bOverwriteFile = false;

	CurrentStreamID = nullptr;

	// If the pixels are FColors, and this is the final pixels buffer, and we're writing PNG, always write out full alpha
	if (PixelData.ImageData->GetType() == EImagePixelType::Color && ImageTask->Format == EImageFormat::PNG && StreamID.Identifiers.OrderIndependentCompareEqual(FinalPixelsID.Identifiers))
	{
		ImageTask->AddPreProcessorToSetAlphaOpaque();
	}

	if (Format == EDesiredImageFormat::EXR)
	{
		ImageTask->CompressionQuality = bEnableCompression ? (int32)EImageCompressionQuality::Default : (int32)EImageCompressionQuality::Uncompressed;
	}
	else
	{
		ImageTask->CompressionQuality = bEnableCompression ? CompressionQuality : 100;
	}

	{
		// Set a callback that is called on the main thread when this file has been written
		TWeakObjectPtr<UUserDefinedImageCaptureProtocol> WeakThis = this;
		ImageTask->OnCompleted = [WeakThis](bool)
		{
			UUserDefinedImageCaptureProtocol* This = WeakThis.Get();
			if (This)
			{
				This->OnFileWritten();
			}
		};
	}

	IImageWriteQueue& ImageWriteQueue = FModuleManager::Get().LoadModuleChecked<IImageWriteQueueModule>("ImageWriteQueue").GetWriteQueue();
	TFuture<bool> DispatchedTask = ImageWriteQueue.Enqueue(MoveTemp(ImageTask));
	if (DispatchedTask.IsValid())
	{
		// If we actually dispatched the task, increment the number of outstanding operations
		ReportOutstandingWork(1);
	}
}

void UUserDefinedImageCaptureProtocol::OnFileWritten()
{
	--NumOutstandingOperations;
}

#undef LOCTEXT_NAMESPACE
