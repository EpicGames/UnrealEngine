// Copyright Epic Games, Inc. All Rights Reserved.

#include "Protocols/VideoCaptureProtocol.h"
#include "HAL/FileManager.h"
#include "Misc/CommandLine.h"
#include "Templates/Casts.h"
#include "Misc/FrameRate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VideoCaptureProtocol)

bool UVideoCaptureProtocol::SetupImpl()
{
#if PLATFORM_UNIX
	UE_LOGF(LogInit, Warning, "Writing movies is not currently supported on Linux");
#endif
	return Super::SetupImpl();
}

void UVideoCaptureProtocol::ConditionallyCreateWriter()
{
#if PLATFORM_LINUX
	static const TCHAR* Extension = TEXT(".unsupp");
	return;
#else
#if PLATFORM_MAC
	static const TCHAR* Extension = TEXT(".mov");
#else
	static const TCHAR* Extension = TEXT(".avi");
#endif

	FString VideoFilename = GenerateFilenameImpl(FFrameMetrics(), Extension);

	if (AVIWriters.Num() && VideoFilename == AVIWriters.Last()->Options.OutputFilename)
	{
		return;
	}

	EnsureFileWritableImpl(VideoFilename);


	FAVIWriterOptions Options;
	Options.OutputFilename = MoveTemp(VideoFilename);
	Options.CaptureFramerateNumerator = CaptureHost->GetCaptureFrameRate().Numerator;
	Options.CaptureFramerateDenominator = CaptureHost->GetCaptureFrameRate().Denominator;
	Options.bSynchronizeFrames = CaptureHost->GetCaptureStrategy().ShouldSynchronizeFrames();
	Options.Width = InitSettings->DesiredSize.X;
	Options.Height = InitSettings->DesiredSize.Y;

	if (bUseCompression)
	{
		Options.CompressionQuality = CompressionQuality / 100.f;
		
		float QualityOverride = 100.f;
		if (FParse::Value( FCommandLine::Get(), TEXT( "-MovieQuality=" ), QualityOverride ))
		{
			Options.CompressionQuality = FMath::Clamp(QualityOverride, 1.f, 100.f) / 100.f;
		}

		Options.CompressionQuality = FMath::Clamp<float>(Options.CompressionQuality.GetValue(), 0.f, 1.f);
	}

	AVIWriters.Emplace(FAVIWriter::CreateInstance(Options));
	AVIWriters.Last()->Initialize();
#endif
}

struct FVideoFrameData : IFramePayload
{
	FFrameMetrics Metrics;
	int32 WriterIndex;
};

FFramePayloadPtr UVideoCaptureProtocol::GetFramePayload(const FFrameMetrics& FrameMetrics)
{
	ConditionallyCreateWriter();

	TSharedRef<FVideoFrameData, ESPMode::ThreadSafe> FrameData = MakeShareable(new FVideoFrameData);
	FrameData->Metrics = FrameMetrics;
	FrameData->WriterIndex = AVIWriters.Num() - 1;
	return FrameData;
}

void UVideoCaptureProtocol::ProcessFrame(FCapturedFrameData Frame)
{
	FVideoFrameData* Payload = Frame.GetPayload<FVideoFrameData>();

	const int32 WriterIndex = Payload->WriterIndex;

	if (WriterIndex >= 0)
	{
		AVIWriters[WriterIndex]->DropFrames(Payload->Metrics.NumDroppedFrames);
		AVIWriters[WriterIndex]->Update(Payload->Metrics.TotalElapsedTime, MoveTemp(Frame.ColorBuffer));
	
		// Finalize previous writers if necessary
		for (int32 Index = 0; Index < Payload->WriterIndex; ++Index)
		{
			TUniquePtr<FAVIWriter>& Writer = AVIWriters[Index];
			if (Writer->IsCapturing())
			{
				Writer->Finalize();
			}
		}
	}
}

void UVideoCaptureProtocol::FinalizeImpl()
{
	for (TUniquePtr<FAVIWriter>& Writer : AVIWriters)
	{
		if (Writer->IsCapturing())
		{
			Writer->Finalize();
		}
	}
	
	AVIWriters.Empty();

	Super::FinalizeImpl();
}

bool UVideoCaptureProtocol::CanWriteToFileImpl(const TCHAR* InFilename, bool bOverwriteExisting) const
{
	// When recording video, if the filename changes (ie due to the shot changing), we create new AVI writers.
	// If we're not overwriting existing filenames we need to check if we're already recording a video of that name,
	// before we can deem whether we can write to a new file (we can always write to a filename we're already writing to)
	if (!bOverwriteExisting)
	{
		for (const TUniquePtr<FAVIWriter>& Writer : AVIWriters)
		{
			if (Writer->Options.OutputFilename == InFilename)
			{
				return true;
			}
		}

		return IFileManager::Get().FileSize(InFilename) == -1;
	}

	return true;
}

