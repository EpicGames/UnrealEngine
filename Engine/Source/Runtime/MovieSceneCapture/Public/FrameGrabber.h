// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/ThreadSafeBool.h"
#include "Slate/SceneViewport.h"
#include "Slate/SlateViewportProvider.h"

class SWindow;

/** A single, managed surface used as a render target resolution destination  */
struct FViewportSurfaceReader
{
	/** Constructor */
	MOVIESCENECAPTURE_API FViewportSurfaceReader(EPixelFormat InPixelFormat, FIntPoint InBufferSize);

	/** Destructor */
	MOVIESCENECAPTURE_API ~FViewportSurfaceReader();

	/** Initialize this reader so that it can be waited on. */
	MOVIESCENECAPTURE_API void Initialize();

	/** Wait for this reader to become available, if it's currently in use */
	MOVIESCENECAPTURE_API void BlockUntilAvailable();

	/** Safely resets the state of the wait event. When doing latent surface reading sometimes we may want to just bail on reading a given frame.
	  * Should only be performed after flushing rendering commands.
	  */
	MOVIESCENECAPTURE_API void Reset();

	/**
	 * Resolve the specified viewport RHI, calling the specified callback with the result.
	 *
	 * @param	BackBuffer		The backbuffer to resolve
	 * @param	Callback 		Callback to call with the locked texture data. This will be called on an undefined thread.
	 */
	MOVIESCENECAPTURE_API void ResolveRenderTarget(FViewportSurfaceReader* RenderToReadback, const FTextureRHIRef& BackBuffer, TFunction<void(FColor*, int32, int32)> Callback);

	/** Get the current size of the texture */
	MOVIESCENECAPTURE_API FIntPoint GetCurrentSize() const;

	/** Set the rectangle within which to read pixels */
	void SetCaptureRect(FIntRect InCaptureRect) { CaptureRect = InCaptureRect; }

	/** Set the window size that we expect from the BackBuffer */
	void SetWindowSize(FIntPoint InWindowSize) { WindowSize = InWindowSize; }

	bool WasEverQueued() const { return bQueuedForCapture; } 

protected:

	/** Set up this surface to the specified width/height */
	MOVIESCENECAPTURE_API void Resize(uint32 Width, uint32 Height);

	/** Whether this surface reader is enabled or not */
	FThreadSafeBool bEnabled;

	/** Optional event that is triggered when the surface is no longer in use */
	FEvent* AvailableEvent;

	/** Texture used to store the resolved render target */
	FTextureRHIRef ReadbackTexture;

	/** The rectangle to read from the surface */
	FIntRect CaptureRect;

	/** In windows mode, the size of the widget with the border */
	FIntPoint WindowSize;

	/** The desired pixel format of the resolved textures */
	EPixelFormat PixelFormat;

	/** Whether this reader is enabled or not. */
	bool bIsEnabled;

	bool bQueuedForCapture;
};

struct IFramePayload
{
	virtual ~IFramePayload() = default;

	/**
	 * Called when the buffer is now available in CPU ram
	 * Return true if you would like to execute the default behavior. (If you return false, GetCapturedFrames will be empty).
	 */
	virtual bool OnFrameReady_RenderThread(FColor* ColorBuffer, FIntPoint BufferSize, FIntPoint TargetSize) const { return true; }
};

typedef TSharedPtr<IFramePayload, ESPMode::ThreadSafe> FFramePayloadPtr;

/** Structure representing a captured frame */
struct FCapturedFrameData
{
	FCapturedFrameData(FIntPoint InBufferSize, FFramePayloadPtr InPayload) : BufferSize(InBufferSize), Payload(MoveTemp(InPayload)) {}

	FCapturedFrameData(FCapturedFrameData&& In) : ColorBuffer(MoveTemp(In.ColorBuffer)), BufferSize(In.BufferSize), Payload(MoveTemp(In.Payload)) {}
	FCapturedFrameData& operator=(FCapturedFrameData&& In){ ColorBuffer = MoveTemp(In.ColorBuffer); BufferSize = In.BufferSize; Payload = MoveTemp(In.Payload); return *this; }

	template<typename T>
	T* GetPayload() { return static_cast<T*>(Payload.Get()); }

	/** The color buffer of the captured frame */
	TArray<FColor> ColorBuffer;

	/** The size of the resulting color buffer */
	FIntPoint BufferSize;

	/** Optional user-specified payload */
	FFramePayloadPtr Payload;
	
private:
	FCapturedFrameData(const FCapturedFrameData& In);
	FCapturedFrameData& operator=(const FCapturedFrameData& In);
};

/**
 * Class responsible for resolving render target data for a specific viewport in an efficient manner
 * Internally, the class uses a fixed array of resolution surfaces, and dispatches rendering commands
 * to resolve the viewport render target into a specific index into this array. This means we can
 * resolve the render target data without having to wait, or flush rendering commands.
 */
class FFrameGrabber
{
public:
	/**
	 * Construct this frame grabber
	 *
	 * @param InViewport			The viewport we are to grab frames for
	 * @param DesiredBufferSize		The desired size of captured frames
	 * @param InPixelFormat			The desired pixel format to store captured frames as
	 * @param InNumSurfaces			The number of destination surfaces contained in our buffer 
	 */
	MOVIESCENECAPTURE_API FFrameGrabber(TSharedRef<FSceneViewport> Viewport, FIntPoint DesiredBufferSize, EPixelFormat InPixelFormat = PF_B8G8R8A8, uint32 NumSurfaces = 3);

	/** Destructor */
	MOVIESCENECAPTURE_API ~FFrameGrabber();

public:

	/** Instruct the frame grabber to start capturing frames */
	MOVIESCENECAPTURE_API void StartCapturingFrames();

	/** Check whether we're capturing frames or not */
	MOVIESCENECAPTURE_API bool IsCapturingFrames() const;

	/** Instruct the frame grabber capture this frame, when it receives an event from slate */
	MOVIESCENECAPTURE_API void CaptureThisFrame(FFramePayloadPtr Payload);

	/** Stop capturing frames */
	MOVIESCENECAPTURE_API void StopCapturingFrames();

	/** Shut down this grabber, ensuring that any threaded operations are finished */
	MOVIESCENECAPTURE_API void Shutdown();

public:

	/** Check whether we have any outstanding frames or not */
	MOVIESCENECAPTURE_API bool HasOutstandingFrames() const;

	/** Retrieve any frames we may have captured */
	MOVIESCENECAPTURE_API TArray<FCapturedFrameData> GetCapturedFrames();

protected:
	
	/** Callback for when a backbuffer is ready for reading (called on render thread) */
	MOVIESCENECAPTURE_API void OnBackBufferReadyToPresentCallback(SWindow& SlateWindow, ISlateViewportProvider& ViewportProvider);

	/** Called when the specified surface index has been locked for reading with the render target data (called on render thread)  */
	MOVIESCENECAPTURE_API void OnFrameReady(int32 SurfaceIndex, FColor* ColorBuffer, int32 Width, int32 Height);

private:

	/** Non-copyable */
	FFrameGrabber(const FFrameGrabber&);
	FFrameGrabber& operator=(const FFrameGrabber&);

	/**
	 * Pointer to the window we want to capture.
	 * Only held for comparison inside OnBackBufferReadyToPresentCallback - never to be dereferenced or cast to an SWindow.
	 * Held as a raw pointer to ensure that no referenc counting occurs from the background thread in OnBackBufferReadyToPresentCallback.
	 */
	void* TargetWindowPtr;

	/** Delegate handle for the OnBackBufferReadyToPresent event */
	FDelegateHandle OnBackBufferReadyToPresent;

	/** Array of captured frames */
	TArray<FCapturedFrameData> CapturedFrames;

	/** Lock to protect the above array */
	mutable FCriticalSection CapturedFramesMutex;

	/** Array of surfaces that we resolve the viewport RHI to. Fixed allocation - should never be resized */
	struct FResolveSurface
	{
		FResolveSurface(EPixelFormat InPixelFormat, FIntPoint BufferSize) : Surface(InPixelFormat, BufferSize) {}

		FFramePayloadPtr Payload;
		FViewportSurfaceReader Surface;
	};
	TArray<FResolveSurface> Surfaces;

	/** Index into the above array to the next surface that we should use - only accessed on main thread */
	int32 CurrentFrameIndex;

	/** The index that we should capture the next rendered slate window into */
	int32 SlateRenderIndex;

	/** The total number of frames we are currently waiting on */
	FThreadSafeCounter OutstandingFrameCount;

	/** Only to be accessed from the render thread - array of frame payloads to be captured from the rendered slate window sorted first to last. */
	TArray<FFramePayloadPtr> RenderThread_PendingFramePayloads;

	int32 FrameGrabLatency;

	/** The current state of the grabber */
	enum class EFrameGrabberState
	{
		Inactive, Active, PendingShutdown
	};
	EFrameGrabberState State;

	/** The desired target size to resolve frames to */
	FIntPoint TargetSize;
};
