// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "IMovieSceneCapture.generated.h"

class FSceneViewport;
struct FMovieSceneCaptureHandle;
struct FMovieSceneCaptureSettings;

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UMovieSceneCaptureInterface : public UInterface
{
public:
	GENERATED_BODY()
};

/** Interface for a movie capture object */
class IMovieSceneCaptureInterface
{
public:
	GENERATED_BODY()

	/** Initialize this capture object by binding it to the specified viewport */
	virtual void Initialize(TSharedPtr<FSceneViewport> Viewport, int32 PIEInstance = -1) = 0;

	/** Instruct this capture to start capturing frames */
	virtual void StartCapturing() = 0;

	/** Shut down this movie capture */
	virtual void Close() = 0;

	/** Get a unique handle to this object */
	virtual FMovieSceneCaptureHandle GetHandle() const = 0;

	/** Access specific movie scene capture settings */
	virtual const FMovieSceneCaptureSettings& GetSettings() const = 0;

	/** Access Frame Number Offset */
	virtual const int32 GetFrameNumberOffset() const = 0;
};
