// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "MovieSceneCapture.h"
#include "LevelCapture.generated.h"

class AActor;
class FSceneViewport;

UCLASS(MinimalAPI)
class ULevelCapture : public UMovieSceneCapture
{
public:
	MOVIESCENECAPTURE_API ULevelCapture(const FObjectInitializer&);

	GENERATED_BODY()

	/** Specifies whether the capture should start immediately, or whether it will be invoked externally (through StartMovieCapture/StopMovieCapture exec commands) */
	UPROPERTY(EditAnywhere, Category=General)
	bool bAutoStartCapture;

	/** Specify a prerequisite actor that must be set up before we start capturing */
	MOVIESCENECAPTURE_API void SetPrerequisiteActor(AActor* Prereq);

	MOVIESCENECAPTURE_API virtual void Initialize(TSharedPtr<FSceneViewport> InViewport, int32 PIEInstance = -1) override;
	MOVIESCENECAPTURE_API virtual void OnTick(float DeltaSeconds) override;
	
private:

	/** Specify a prerequisite actor that must be set up before we start capturing */
	TWeakObjectPtr<AActor> PrerequisiteActor;

	/** Copy of the ID from PrerequisiteActor. Required because JSON serialization exports the path of the object, rather that its GUID */
	UPROPERTY()
	FGuid PrerequisiteActorId;

	/** PIE instance index that we are capturing with */
	int32 PIECaptureInstance;
};
