// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptInterface.h"
#include "MovieSceneCaptureProtocolBase.h"
#include "MovieSceneCaptureHandle.h"
#include "MovieSceneCaptureSettings.h"
#include "IMovieSceneCapture.h"
#include "Scalability.h"
#include "UObject/SoftObjectPath.h"
#include "MovieSceneCapture.generated.h"

class FJsonObject;
class FSceneViewport;

/** Structure used to cache various metrics for our capture */
struct FCachedMetrics
{
	FCachedMetrics() : Width(0), Height(0), Frame(0), ElapsedSeconds(0.f), PreviousFrame(INDEX_NONE) {}

	/** The width/Height of the frame */
	int32 Width, Height;
	/** The current frame number */
	int32 Frame;
	/** The number of seconds that have elapsed */
	float ElapsedSeconds;
	/** The previous frame number */
	int32 PreviousFrame;
};

/** Class responsible for capturing scene data */
UCLASS(config=EditorPerProjectUserSettings, PerObjectConfig, BlueprintType, MinimalAPI)
class UMovieSceneCapture : public UObject, public IMovieSceneCaptureInterface, public ICaptureProtocolHost
{
public:
	MOVIESCENECAPTURE_API UMovieSceneCapture(const FObjectInitializer& Initializer);

	GENERATED_BODY()

	/** This name is used by the UI to save/load a specific instance of the settings from config that doesn't affect the CDO which would affect scripting environments. */
	static MOVIESCENECAPTURE_API const FName MovieSceneCaptureUIName;

	MOVIESCENECAPTURE_API virtual void PostInitProperties() override;

public:

	// Begin IMovieSceneCaptureInterface
	MOVIESCENECAPTURE_API virtual void Initialize(TSharedPtr<FSceneViewport> InSceneViewport, int32 PIEInstance = -1) override;
	virtual void StartCapturing() { StartCapture(); }
	virtual void Close() override { Finalize(); }
	virtual FMovieSceneCaptureHandle GetHandle() const override { return Handle; }
	const FMovieSceneCaptureSettings& GetSettings() const override { return Settings; }
	const int32 GetFrameNumberOffset() const override { return FrameNumberOffset; }
	// End IMovieSceneCaptureInterface

	/** Load save from config helpers */
	MOVIESCENECAPTURE_API virtual void LoadFromConfig();
	MOVIESCENECAPTURE_API virtual void SaveToConfig();

	/** Serialize additional json data for this capture */
	MOVIESCENECAPTURE_API void SerializeJson(FJsonObject& Object);

	/** Deserialize additional json data for this capture */
	MOVIESCENECAPTURE_API void DeserializeJson(const FJsonObject& Object);

protected:

	/** Custom, additional json serialization */
	virtual void SerializeAdditionalJson(FJsonObject& Object){}

	/** Custom, additional json deserialization */
	virtual void DeserializeAdditionalJson(const FJsonObject& Object){}

	/** Returns true if this is currently the audio pass, or if an audio pass is not needed. Shorthand for checking if we're in a state where we should finish capture. */
	MOVIESCENECAPTURE_API virtual bool IsAudioPassIfNeeded() const;
public:

	/** The type of capture protocol to use for image data */
	UPROPERTY(config, EditAnywhere, NoClear, Category=CaptureSettings, DisplayName="Image Output Format", meta=(MetaClass="/Script/MovieSceneCapture.MovieSceneImageCaptureProtocolBase", HideViewOptions, ShowDisplayNames))
	FSoftClassPath ImageCaptureProtocolType;
	
	/** The type of capture protocol to use for audio data. */
	UPROPERTY(config, EditAnywhere, NoClear, Category=CaptureSettings, DisplayName="Audio Output Format", meta=(MetaClass="/Script/MovieSceneCapture.MovieSceneAudioCaptureProtocolBase", HideViewOptions, ShowDisplayNames))
	FSoftClassPath AudioCaptureProtocolType;

	/** Capture protocol responsible for actually capturing frame data */
	UPROPERTY(VisibleAnywhere, Category=CaptureSettings, Transient, Instanced)
	TObjectPtr<UMovieSceneImageCaptureProtocolBase> ImageCaptureProtocol;
	
	UPROPERTY(VisibleAnywhere, Category = CaptureSettings, Transient, Instanced)
	TObjectPtr<UMovieSceneAudioCaptureProtocolBase> AudioCaptureProtocol;

	/** Settings that define how to capture */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=CaptureSettings, meta=(ShowOnlyInnerProperties))
	FMovieSceneCaptureSettings Settings;

	/** Whether to capture the movie in a separate process or not */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=General, AdvancedDisplay)
	bool bUseSeparateProcess;

	/** When enabled, the editor will shutdown when the capture starts */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=General, AdvancedDisplay, meta=(EditCondition=bUseSeparateProcess))
	bool bCloseEditorWhenCaptureStarts;

	/** Additional command line arguments to pass to the external process when capturing */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=General, AdvancedDisplay, meta=(EditCondition=bUseSeparateProcess))
	FString AdditionalCommandLineArguments;

	/** Command line arguments inherited from this process */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, transient, Category=General, AdvancedDisplay, meta=(EditCondition=bUseSeparateProcess))
	FString InheritedCommandLineArguments;

	/** Event that is fired after we've finished capturing */
	DECLARE_EVENT( UMovieSceneCapture, FOnCaptureFinished );
	FOnCaptureFinished& OnCaptureFinished() { return OnCaptureFinishedDelegate; }

public:

	/** Access this object's cached metrics */
	const FCachedMetrics& GetMetrics() const { return CachedMetrics; }

	/** Access the capture protocol we are using */
	UFUNCTION(BlueprintCallable, Category=Capture)
	UMovieSceneCaptureProtocolBase* GetImageCaptureProtocol() { return ImageCaptureProtocol; }
	UFUNCTION(BlueprintCallable, Category=Capture)
	UMovieSceneCaptureProtocolBase* GetAudioCaptureProtocol() { return AudioCaptureProtocol; }
	
	
	UFUNCTION(BlueprintCallable, Category=Capture)
	MOVIESCENECAPTURE_API void SetImageCaptureProtocolType(TSubclassOf<UMovieSceneCaptureProtocolBase> ProtocolType);
	UFUNCTION(BlueprintCallable, Category=Capture)
	MOVIESCENECAPTURE_API void SetAudioCaptureProtocolType(TSubclassOf<UMovieSceneCaptureProtocolBase> ProtocolType);

public:

	/** Starts warming up.  May be optionally called before StartCapture().  This can be used to start rendering frames early, before
	    any files are captured or written out */
	MOVIESCENECAPTURE_API void StartWarmup();

	/** Initialize the capture so that it is able to start capturing frames */
	MOVIESCENECAPTURE_API void StartCapture();

	/** Indicate that this frame should be captured - must be called before the movie scene capture is ticked */
	MOVIESCENECAPTURE_API void CaptureThisFrame(float DeltaSeconds);

	/** Automatically finalizes the capture when all currently pending frames are dealt with */
	MOVIESCENECAPTURE_API void FinalizeWhenReady();

	/** Check whether we should automatically finalize this capture */
	MOVIESCENECAPTURE_API bool ShouldFinalize() const;

	/** Finalize the capturing process, assumes all frames have been processed. */
	MOVIESCENECAPTURE_API void Finalize();

public:

	/** Called at the end of a frame, before a frame is presented by slate */
	MOVIESCENECAPTURE_API void Tick(float DeltaSeconds);

	// ICaptureProtocolHost interface
	/** Resolve the specified format using the user supplied formatting rules. */
	MOVIESCENECAPTURE_API FString ResolveFileFormat(const FString& Format, const FFrameMetrics& FrameMetrics) const;

	/** Estimate how long our duration is going to be for pre-allocation purposes. */
	double GetEstimatedCaptureDurationSeconds() const { return 0.0; }


	virtual FFrameRate GetCaptureFrameRate() const { return Settings.GetFrameRate(); }
	virtual const ICaptureStrategy& GetCaptureStrategy() const { return *CaptureStrategy; }
	// ~ICaptureProtocolHost interface

protected:
	/** Add additional format mappings to be used when generating filenames */
	virtual void AddFormatMappings(TMap<FString, FStringFormatArg>& OutFormatMappings, const FFrameMetrics& FrameMetrics) const {}


	/** Initialize the settings structure for the current capture type */
	MOVIESCENECAPTURE_API void InitializeCaptureProtocols();
	
	MOVIESCENECAPTURE_API void ForciblyReinitializeCaptureProtocols();

	/** Called at the end of a frame, before a frame is presented by slate */
	virtual void OnTick(float DeltaSeconds) { CaptureThisFrame(DeltaSeconds); }
protected:

#if WITH_EDITOR
	MOVIESCENECAPTURE_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

protected:
	/** Strategy used for capture (real-time/fixed-time-step) */
	TSharedPtr<ICaptureStrategy> CaptureStrategy;
	/** The settings we will use to set up the capture protocol */
	TOptional<FCaptureProtocolInitSettings> InitSettings;
	/** Whether we should automatically attempt to capture frames every tick or not */
	bool bFinalizeWhenReady;
	/** Our unique handle, used for external representation without having to link to the MovieSceneCapture module */
	FMovieSceneCaptureHandle Handle;
	/** Cached metrics for this capture operation */
	FCachedMetrics CachedMetrics;
	/** Format mappings used for generating filenames */
	TMap<FString, FStringFormatArg> FormatMappings;
	/** Whether we have started capturing or not */
	bool bCapturing;
	/** If we're currently doing an audio pass or not */
	bool bIsAudioCapturePass;

	/** Frame number index offset when saving out frames.  This is used to allow the frame numbers on disk to match
	    what they would be in the authoring application, rather than a simple 0-based sequential index */
	int32 FrameNumberOffset;
	/** Event that is triggered when capturing has finished */
	FOnCaptureFinished OnCaptureFinishedDelegate;
	/** Cached quality levels */
	Scalability::FQualityLevels CachedQualityLevels;
};

/** A strategy that employs a fixed frame time-step, and as such never drops a frame. Potentially accelerated. */
struct FFixedTimeStepCaptureStrategy : ICaptureStrategy
{
	MOVIESCENECAPTURE_API FFixedTimeStepCaptureStrategy(FFrameRate InFrameRate);

	MOVIESCENECAPTURE_API virtual void OnInitialize() override;
	MOVIESCENECAPTURE_API virtual void OnStop() override;
	MOVIESCENECAPTURE_API virtual bool ShouldPresent(double CurrentTimeSeconds, uint32 FrameIndex) const override;
	MOVIESCENECAPTURE_API virtual int32 GetDroppedFrames(double CurrentTimeSeconds, uint32 FrameIndex) const override;

private:
	FFrameRate FrameRate;
};

/** A capture strategy that captures in real-time, potentially dropping frames to maintain a stable constant framerate video. */
struct FRealTimeCaptureStrategy : ICaptureStrategy
{
	MOVIESCENECAPTURE_API FRealTimeCaptureStrategy(FFrameRate InFrameRate);

	MOVIESCENECAPTURE_API virtual void OnInitialize() override;
	MOVIESCENECAPTURE_API virtual void OnStop() override;
	virtual bool ShouldSynchronizeFrames() const override { return false; }
	MOVIESCENECAPTURE_API virtual bool ShouldPresent(double CurrentTimeSeconds, uint32 FrameIndex) const override;
	MOVIESCENECAPTURE_API virtual int32 GetDroppedFrames(double CurrentTimeSeconds, uint32 FrameIndex) const override;

private:
	double NextPresentTimeS, FrameLength;
};
