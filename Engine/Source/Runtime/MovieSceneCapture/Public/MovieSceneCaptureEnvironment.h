// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "MovieSceneCaptureEnvironment.generated.h"

class UMovieSceneCaptureProtocolBase;

UCLASS(MinimalAPI)
class UMovieSceneCaptureEnvironment : public UObject
{
public:
	GENERATED_BODY()

	/** Get the frame number of the current capture */
	UFUNCTION(BlueprintPure, Category="Cinematics|Capture")
	static MOVIESCENECAPTURE_API int32 GetCaptureFrameNumber();

	/** Get the total elapsed time of the current capture in seconds */
	UFUNCTION(BlueprintPure, Category="Cinematics|Capture")
	static MOVIESCENECAPTURE_API float GetCaptureElapsedTime();

	/**
	 * Return true if there is any capture currently active (even in a warm-up state).
	 * Useful for checking whether to do certain operations in BeginPlay
	 */
	UFUNCTION(BlueprintCallable, Category="Cinematics|Capture")
	static MOVIESCENECAPTURE_API bool IsCaptureInProgress();

	/**
	 * Attempt to locate a capture protocol - may not be in a capturing state
	 */
	UFUNCTION(BlueprintCallable, Category="Cinematics|Capture")
	static MOVIESCENECAPTURE_API UMovieSceneImageCaptureProtocolBase* FindImageCaptureProtocol();

	/**
	* Attempt to locate a capture protocol - may not be in a capturing state
	*/
	UFUNCTION(BlueprintCallable, Category = "Cinematics|Capture")
	static MOVIESCENECAPTURE_API UMovieSceneAudioCaptureProtocolBase* FindAudioCaptureProtocol();
};
