// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** A handle to a movie scene capture implementation */
struct FMovieSceneCaptureHandle
{
	FMovieSceneCaptureHandle() : ID(0) {}

	/** Equality operator */
	friend bool operator==(const FMovieSceneCaptureHandle& A, const FMovieSceneCaptureHandle& B) { return A.ID == B.ID; }
	
	/** Inequality operator */
	friend bool operator!=(const FMovieSceneCaptureHandle& A, const FMovieSceneCaptureHandle& B) { return A.ID != B.ID; }

	/** Check if this handle is valid */
	bool IsValid() const { return ID > 0; }

protected:
	/** Unique identifier for this handle */
	uint32 ID;
};
