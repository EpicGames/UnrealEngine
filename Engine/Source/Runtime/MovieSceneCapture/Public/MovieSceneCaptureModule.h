// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneCaptureHandle.h"

class FSceneViewport;
class IMovieSceneCaptureInterface;

MOVIESCENECAPTURE_API DECLARE_LOG_CATEGORY_EXTERN(LogMovieSceneCapture, Log, All);

class IMovieSceneCaptureModule : public IModuleInterface
{
public:
	static IMovieSceneCaptureModule& Get()
	{
		static const FName ModuleName("MovieSceneCapture");
		return FModuleManager::LoadModuleChecked< IMovieSceneCaptureModule >(ModuleName);
	}

	virtual IMovieSceneCaptureInterface* InitializeFromCommandLine() = 0;
	virtual IMovieSceneCaptureInterface* CreateMovieSceneCapture(TSharedPtr<FSceneViewport> Viewport) = 0;
	virtual IMovieSceneCaptureInterface* GetFirstActiveMovieSceneCapture() = 0;
	
	virtual IMovieSceneCaptureInterface* RetrieveMovieSceneInterface(FMovieSceneCaptureHandle Handle) = 0;
	virtual void DestroyMovieSceneCapture(FMovieSceneCaptureHandle Handle) = 0;

	virtual void DestroyAllActiveCaptures() = 0;
	virtual bool IsStereoAllowed() = 0;
};

