// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "MovieSceneCaptureSettings.h"
#include "MovieSceneCaptureProtocolBase.generated.h"

class FSceneViewport;

struct FFrameRate;
struct FFrameMetrics;
struct ICaptureProtocolHost;
struct FMovieSceneCaptureSettings;


UENUM()
enum class EMovieSceneCaptureProtocolState : uint8
{
	/** The protocol is idle, and has not even been initialized */
	Idle,
	/** The protocol has been initialized (and bound to a viewport) but is not capturing frames yet */
	Initialized,
	/** The protocol has been initialized, bound to a viewport and is capturing data */
	Capturing,
	/** The protocol has finished capturing data, and is pending finalization */
	Finalizing,
};

/** Structure used to initialize a capture protocol */
struct FCaptureProtocolInitSettings
{
	/**~ @todo: add ability to capture a sub-rectangle */

	/** Capture from a slate viewport, using the specified custom protocol settings */
	MOVIESCENECAPTURE_API static FCaptureProtocolInitSettings FromSlateViewport(TSharedRef<FSceneViewport> InSceneViewport);
	
	/** The slate viewport we should capture from */
	TSharedPtr<FSceneViewport> SceneViewport;
	/** The desired size of the captured frames */
	FIntPoint DesiredSize;

private:
	/** Private construction to ensure use of static init methods */
	FCaptureProtocolInitSettings() {}
};


/**
 * A capture protocol responsible for dealing with captured frames using some custom method (writing out to disk, streaming, etc)
 *
 * A typical process for capture consits of the following process:
 *     Setup -> [ Warm up -> [ Capture Frame ] ] -> Begin Finalize -> [ HasFinishedProcessing ] -> Finalize
 */
UCLASS(config=EditorPerProjectUserSettings, PerObjectConfig, Abstract, MinimalAPI)
class UMovieSceneCaptureProtocolBase : public UObject
{
public:

	GENERATED_BODY()

	MOVIESCENECAPTURE_API UMovieSceneCaptureProtocolBase(const FObjectInitializer& ObjInit);

public:

	/**
	 * Get the current state of this capture protocol
	 */
	UFUNCTION(BlueprintCallable, Category=Capture)
	EMovieSceneCaptureProtocolState GetState() const { return State; }

	/**
	 * Check whether we can capture a frame from this protocol
	 */
	UFUNCTION(BlueprintCallable, Category=Capture)
	bool IsCapturing() const { return State == EMovieSceneCaptureProtocolState::Capturing || bFrameRequested[GFrameCounter%2] == true; }

	/**
	 * Setup this capture protocol
	 *
	 * @param InSettings		The initial initialization settings to use for the capture
	 * @param Host				The client that is initializing this protocol
	 */
	MOVIESCENECAPTURE_API bool Setup(const FCaptureProtocolInitSettings& InSettings, const ICaptureProtocolHost* Host);

	/**
	 * Get the UWorld associated with this Capture Protocol. This is not valid until
	 * Setup has been called with a valid Slate viewport. Will return nullptr when
	 * the protocol has been created but the game world is not running (ie: in UI).
	 */
	MOVIESCENECAPTURE_API virtual class UWorld* GetWorld() const override;
public:

	/**
	 * Called on the main thread before the movie capture itself is updated to reset per-frame state
	 */
	MOVIESCENECAPTURE_API void PreTick();

	/**
	 * Called on the main thread to do any additional processing
	 */
	MOVIESCENECAPTURE_API void Tick();

	/**
	 * Start warming up this capture protocol - called any time the process enters a warming-up state
	 */
	MOVIESCENECAPTURE_API void WarmUp();

	/**
	 * Called when this protocol should start capturing
	 *
	 * @return true if the operation was successful, false otherwise
	 */
	MOVIESCENECAPTURE_API bool StartCapture();

	/**
	 * Instruct this protocol to capture a frame relating to the specified metrics
	 *
	 * @param FrameMetrics		Frame metrics relating to the current frame
	 * @param Host				The client that is initializing this protocol
	 */
	MOVIESCENECAPTURE_API void CaptureFrame(const FFrameMetrics& FrameMetrics);

	/**
	 * Called when we have finished capturing and we should start finalizing the capture
	 */
	MOVIESCENECAPTURE_API void BeginFinalize();

	/**
	 * Check whether this protocol has any processing left to do, or whether it should be finalized.
	 * Only called when the capture has been asked to end.
	 */
	MOVIESCENECAPTURE_API bool HasFinishedProcessing() const;

	/**
	 * Called when this protocol should tear down and finalize all its processing. Only called if HasFinishedProcessing is true.
	 */
	MOVIESCENECAPTURE_API void Finalize();

	/**
	 * Called when generating formatting filename to add any additional format mappings
	 *
	 * @param FormatMappings	Map to add additional format rules to
	 */
	MOVIESCENECAPTURE_API void AddFormatMappings(TMap<FString, FStringFormatArg>& FormatMappings) const;

	/**
	 * Called when this protocol has been released
	 */
	MOVIESCENECAPTURE_API void OnReleaseConfig(FMovieSceneCaptureSettings& InSettings);

	/**
	 * Called when this protocol has been loaded
	 */
	MOVIESCENECAPTURE_API void OnLoadConfig(FMovieSceneCaptureSettings& InSettings);

	/**
	 * Test whether this capture protocol thinks the file should be written to. Only called when we're not overwriting existing files.
	 * By default, we simply test for the file's existence, however this can be overridden to afford complex behaviour like
	 * writing out multiple video files for different file names
	 * @param InFilename 			The filename to test
	 * @param bOverwriteExisting	Whether we are allowed to overwrite existing files
	 * @return Whether we should deem this file writable or not
	 */
	MOVIESCENECAPTURE_API bool CanWriteToFile(const TCHAR* InFilename, bool bOverwriteExisting) const;

protected:

	/**
	 * Called once at the start of the capture process (before any warmup) to set up anything required for the capture.
	 */
	virtual bool SetupImpl()
	{
		return true;
	}

	/**
	 * Called on the main thread before the movie capture itself is updated to reset per-frame state
	 */
	virtual void PreTickImpl() {}

	/**
	 * Called on the main thread to do any additional processing
	 */
	virtual void TickImpl() {}

	/**
	 * Start warming up this capture protocol
	 */
	virtual void WarmUpImpl() {}

	/**
	 * Start capturing
	 *
	 * @return true if the operation was successful, false otherwise
	 */
	virtual bool StartCaptureImpl()
	{
		return true;
	}

	/**
	 * Instruct this protocol to capture a frame relating to the specified metrics
	 *
	 * @param FrameMetrics		Frame metrics relating to the current frame
	 */
	virtual void CaptureFrameImpl(const FFrameMetrics& FrameMetrics) {}

	/**
	 * Pause capturing
	 */
	virtual void PauseCaptureImpl() {}

	/**
	 * Called when we have finished capturing and we should start finalizing the capture
	 */
	virtual void BeginFinalizeImpl() {}

	/**
	 * Check whether this protocol has any processing left to do, or whether it should be finalized.
	 * Only called when the capture has been asked to end.
	 */
	virtual bool HasFinishedProcessingImpl() const { return true; }

	/**
	 * Called when we have finished capturing
	 */
	virtual void FinalizeImpl() {}

	/**
	 * Called when generating formatting filename to add any additional format mappings
	 *
	 * @param FormatMappings	Map to add additional format rules to
	 */
	virtual void AddFormatMappingsImpl(TMap<FString, FStringFormatArg>& FormatMappings) const {}

	/**
	 * Called when this protocol has been released
	 */
	virtual void OnReleaseConfigImpl(FMovieSceneCaptureSettings& InSettings) {}

	/**
	 * Called when this protocol has been loaded
	 */
	virtual void OnLoadConfigImpl(FMovieSceneCaptureSettings& InSettings) {}

	/**
	 * Test whether this capture protocol thinks the file should be written to. Only called when we're not overwriting existing files.
	 * By default, we simply test for the file's existence, however this can be overridden to afford complex behaviour like
	 * writing out multiple video files for different file names
	 * @param InFilename 			The filename to test
	 * @param bOverwriteExisting	Whether we are allowed to overwrite existing files
	 * @return Whether we should deem this file writable or not
	 */
	MOVIESCENECAPTURE_API virtual bool CanWriteToFileImpl(const TCHAR* InFilename, bool bOverwriteExisting) const;
	MOVIESCENECAPTURE_API virtual FString GenerateFilenameImpl(const FFrameMetrics& FrameMetrics, const TCHAR* Extension, const FString* NameFormatString = nullptr) const;
	MOVIESCENECAPTURE_API void EnsureFileWritableImpl(const FString& File) const;

protected:

	/** Initialization settings */
	TOptional<FCaptureProtocolInitSettings> InitSettings;

	/** The capture host that is owns this protocol */
	const ICaptureProtocolHost* CaptureHost;

private:

	/** The current state of the protocol */
	UPROPERTY(transient)
	EMovieSceneCaptureProtocolState State;

	/** Double buffer of values tracking whether we are capturing changes based on GFrameCounter */
	bool bFrameRequested[2];
};

/**
* A class to inherit from for image capture protocols. Used to filter the UI for protocols used on the image capture pass.
*/
UCLASS(config = EditorPerProjectUserSettings, PerObjectConfig, Abstract, MinimalAPI)
class  UMovieSceneImageCaptureProtocolBase : public UMovieSceneCaptureProtocolBase
{
	GENERATED_BODY()

public:
	UMovieSceneImageCaptureProtocolBase(const FObjectInitializer& ObjInit)
		: UMovieSceneCaptureProtocolBase(ObjInit)
	{}

};

/**
* A class to inherit from for audio capture protocols. Used to filter the UI for protocols used on the audio capture pass.
*/
UCLASS(config = EditorPerProjectUserSettings, PerObjectConfig, Abstract, MinimalAPI)
class UMovieSceneAudioCaptureProtocolBase : public UMovieSceneCaptureProtocolBase
{
	GENERATED_BODY()

public:
	UMovieSceneAudioCaptureProtocolBase(const FObjectInitializer& ObjInit)
		: UMovieSceneCaptureProtocolBase(ObjInit)
	{}
};

/** Metrics that correspond to a particular frame */
USTRUCT(BlueprintType)
struct FFrameMetrics
{
	GENERATED_BODY()

	/** Default construction */
	FFrameMetrics() : TotalElapsedTime(0), FrameDelta(0), FrameNumber(0), NumDroppedFrames(0) {}

	FFrameMetrics(float InTotalElapsedTime, float InFrameDelta, int32 InFrameNumber, int32 InNumDroppedFrames)
		: TotalElapsedTime(InTotalElapsedTime), FrameDelta(InFrameDelta), FrameNumber(InFrameNumber), NumDroppedFrames(InNumDroppedFrames)
	{
	}

	/** The total amount of time, in seconds, since the capture started */
	UPROPERTY(BlueprintReadOnly, Category=Capture)
	float TotalElapsedTime;

	/** The total amount of time, in seconds, that this specific frame took to render (not accounting for dropped frames) */
	UPROPERTY(BlueprintReadOnly, Category=Capture)
	float FrameDelta;

	/** The index of this frame from the start of the capture, including dropped frames */
	UPROPERTY(BlueprintReadOnly, Category=Capture)
	int32 FrameNumber;

	/** The number of frames we dropped in-between this frame, and the last one we captured */
	UPROPERTY(BlueprintReadOnly, Category=Capture)
	int32 NumDroppedFrames;
};


/** Interface that defines when to capture or drop frames */
struct ICaptureStrategy
{
	virtual ~ICaptureStrategy()= default;

	virtual void OnInitialize() = 0;
	virtual void OnStop() = 0;
	virtual bool ShouldSynchronizeFrames() const { return true; }

	virtual bool ShouldPresent(double CurrentTimeSeconds, uint32 FrameIndex) const = 0;
	virtual int32 GetDroppedFrames(double CurrentTimeSeconds, uint32 FrameIndex) const = 0;
};


/** Interface to be implemented by any class using an UMovieSceneCaptureProtocolBase instance*/
struct ICaptureProtocolHost
{
	/** Get shared settings for the capture */
	virtual const FMovieSceneCaptureSettings& GetSettings() const = 0;

	/** Access Frame number index offset when saving out frames. */
	virtual const int32 GetFrameNumberOffset() const = 0;

	/** Get the capture frequency */
	virtual FFrameRate GetCaptureFrameRate() const = 0;

	/** Access the host's capture strategy */
	virtual const ICaptureStrategy& GetCaptureStrategy() const = 0;

	/** Ask the host to resolve the format string for a file name. */
	virtual FString ResolveFileFormat(const FString& Format, const FFrameMetrics& FrameMetrics) const = 0;

	/** Ask the host to inform us of how long the capture duration is expected to be. Should only be used as an estimate
	 * due to the possible complexities in calculating the duration due to handle frames, warmups, etc.
	 */
	virtual double GetEstimatedCaptureDurationSeconds() const = 0;
};


