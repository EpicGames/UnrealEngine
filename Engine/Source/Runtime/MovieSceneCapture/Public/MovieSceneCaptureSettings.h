// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "Misc/FrameRate.h"
#include "MovieSceneCaptureSettings.generated.h"


/** Structure representing a capture resolution */
USTRUCT(BlueprintType)
struct FCaptureResolution
{
	FCaptureResolution(int32 InX = 0, int32 InY = 0) : ResX(InX), ResY(InY) {}
	
	GENERATED_BODY()

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=Resolution, meta=(ClampMin = 16, ClampMax=7680))
	int32 ResX;

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=Resolution, meta=(ClampMin = 16, ClampMax=7680))
	int32 ResY;
};

/** Common movie-scene capture settings */
USTRUCT(BlueprintType)
struct FMovieSceneCaptureSettings
{
	MOVIESCENECAPTURE_API FMovieSceneCaptureSettings();

	GENERATED_BODY()

	/** The directory to output the captured file(s) in */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=General, meta=(RelativePath))
	FDirectoryPath OutputDirectory;

	/** Optional game mode to override the map's default game mode with.  This can be useful if the game's normal mode displays UI elements or loading screens that you don't want captured. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=General, AdvancedDisplay)
	TSubclassOf<class AGameModeBase> GameModeOverride;

	/** The format to use for the resulting filename. Extension will be added automatically. Any tokens of the form {token} will be replaced with the corresponding value:
	 * {fps}		- The captured framerate
	 * {frame}		- The current frame number (only relevant for image sequences)
	 * {width}		- The width of the captured frames
	 * {height}		- The height of the captured frames
	 * {world}		- The name of the current world
	 * {quality}	- The image compression quality setting
	 * {material}   - The material/render pass
	 * {shot}       - The name of the level sequence asset shot being played
	 * {sequence}   - The name of the level sequence asset being played
	 * {camera}     - The name of the current camera
	 * {date}       - The date in the format of {year}.{month}.{day}
	 * {year}       - The current year
	 * {month}      - The current month
	 * {day}        - The current day
	 * {time}       - The current time in the format of hours.minutes.seconds
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=General, DisplayName="Filename Format")
	FString OutputFormat;

	/** Whether to overwrite existing files or not */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=General, AdvancedDisplay )
	bool bOverwriteExisting;

	/** True if frame numbers in the output files should be relative to zero, rather than the actual frame numbers in the originating animation content. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=General, AdvancedDisplay)
	bool bUseRelativeFrameNumbers;

	/** Number of frame handles to include for each shot */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=Sequence, AdvancedDisplay, meta=(ClampMin=0, UIMin=0))
	int32 HandleFrames;

	/** Filename extension for movies referenced in the XMLs/EDLs */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=Sequence, AdvancedDisplay)
	FString MovieExtension;

	/** How much to zero-pad frame numbers on filenames */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=General, AdvancedDisplay, meta=(ClampMin=0, UIMin=0))
	uint8 ZeroPadFrameNumbers;

	/** The sequence's frame rate at which to capture if "Use Custom Frame Rate" is not enabled */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = CaptureSettings, meta = (ClampMin = 1, UIMin = 1, ClampMax = 200, UIMax = 200))
	FFrameRate FrameRate;

	/** Specify using the custom frame rate as opposed to the sequence's display rate */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=CaptureSettings)
	bool bUseCustomFrameRate;

	/** The custom frame rate at which to capture if "Use Custom Frame Rate" is enabled */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=CaptureSettings, meta = (ClampMin = 1, UIMin = 1, ClampMax = 200, UIMax = 200, EditCondition=bUseCustomFrameRate))
	FFrameRate CustomFrameRate;

	/** The resolution at which to capture */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=CaptureSettings, meta=(ShowOnlyInnerProperties))
	FCaptureResolution Resolution;
	
	/** Whether to texture streaming should be enabled while capturing.  Turning off texture streaming may cause much more memory to be used, but also reduces the chance of blurry textures in your captured video. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=CaptureSettings, AdvancedDisplay)
	bool bEnableTextureStreaming;

	/** Whether to enable cinematic engine scalability settings */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=Cinematic)
	bool bCinematicEngineScalability;

	/** Whether to enable cinematic mode whilst capturing */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=Cinematic)
	bool bCinematicMode;

	/** Whether to allow player movement whilst capturing */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=Cinematic, AdvancedDisplay, meta=(EditCondition="bCinematicMode"))
	bool bAllowMovement;

	/** Whether to allow player rotation whilst capturing */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=Cinematic, AdvancedDisplay, meta=(EditCondition="bCinematicMode"))
	bool bAllowTurning;

	/** Whether to show the local player whilst capturing */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=Cinematic, AdvancedDisplay, meta=(EditCondition="bCinematicMode"))
	bool bShowPlayer;

	/** Whether to show the in-game HUD whilst capturing */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=Cinematic, AdvancedDisplay, meta=(EditCondition="bCinematicMode"))
	bool bShowHUD;

	/** Whether to use the path tracer (if supported) to render the scene */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = Cinematic, AdvancedDisplay)
	bool bUsePathTracer;

	/** Number of sampler per pixel to be used when rendering the scene with the path tracer (if supported) */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = Cinematic, AdvancedDisplay, meta = (EditCondition = "bUsePathTracer", ClampMin = 1, UIMin = 1, UIMax = 4096))
	int32 PathTracerSamplePerPixel;

	/** Get which frame rate to use */
	FFrameRate GetFrameRate() const { return bUseCustomFrameRate ? CustomFrameRate : FrameRate; }
};
