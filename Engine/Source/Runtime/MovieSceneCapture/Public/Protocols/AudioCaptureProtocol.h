// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "Slate/SceneViewport.h"
#include "AudioDevice.h"
#include "AudioMixerBlueprintLibrary.h"
#include "MovieSceneCaptureModule.h"
#include "MovieSceneCaptureProtocolBase.h"
#include "AudioCaptureProtocol.generated.h"

/**
 * This is a null audio capture implementation which skips capturing audio. The MovieSceneCapture is explicitly
 * aware of this type and will skip processing an audio pass if this is specified.
 */
UCLASS(meta = (DisplayName = "No Audio", CommandLineID = "NullAudio"), MinimalAPI)
class UNullAudioCaptureProtocol : public UMovieSceneAudioCaptureProtocolBase
{
	GENERATED_BODY()
};

/**
* This is an experimental audio capture implementation which captures the final output from the master submix.
* This requires that your sequence can be played back in real-time (when rendering is disabled). 
* If the sequence evaluation hitches the audio will become desynchronized due to their being more time passed 
* in real time (platform time) than in the sequence itself.
*/
UCLASS(MinimalApi, meta = (DisplayName = "Master Audio Submix", CommandLineID = "MasterAudioSubmix"))
class UMasterAudioSubmixCaptureProtocol : public UMovieSceneAudioCaptureProtocolBase
{
public:
	GENERATED_BODY()

	UMasterAudioSubmixCaptureProtocol(const FObjectInitializer& Init)
		: UMovieSceneAudioCaptureProtocolBase(Init)
		, TotalGameRecordingTime(0.0)
		, TotalPlatformRecordingTime(0.0)
		, GameRecordingStartTime(0.0)
		, PlatformRecordingStartTime(0.0)
		, bHasSetup(false)
	{
		// Match default format for video captures.
		FileName = TEXT("{world}");
	}

	/**~ UMovieSceneCaptureProtocolBase Implementation */
	virtual bool SetupImpl() override
	{

		return true;
	}

	virtual bool StartCaptureImpl() override
	{
		if (!bHasSetup)
		{
			// This is called every time we want to resume capturing audio.
			if (GetWorld() && GetWorld()->GetGameViewport())
			{
				// Disable rendering so we save all the render thread/GPU overhead
				GetWorld()->GetGameViewport()->bDisableWorldRendering = true;
			}
		
			UAudioMixerBlueprintLibrary::StartRecordingOutput(GetWorld(), CaptureHost->GetEstimatedCaptureDurationSeconds());
			bHasSetup = true;
		}
		else
		{
			UE_LOGF(LogTemp, Warning, "Audio Recording Resumed");
			UAudioMixerBlueprintLibrary::ResumeRecordingOutput(GetWorld());
		}


		GameRecordingStartTime = GetWorld()->TimeSeconds;
		PlatformRecordingStartTime = FPlatformTime::Seconds();
		return true;
	}

	virtual void PauseCaptureImpl() override
	{
		// Pause the audio capture so we don't incorrectly capture audio for durations that we're not capturing frames for.
		UAudioMixerBlueprintLibrary::PauseRecordingOutput(GetWorld());

		// Stop all sounds currently playing on the Audio Device. This helps kill looping or long audio clips. When the sequence evaluates again,
		// these clips will resume play at the correct location.
		if (FAudioDeviceHandle AudioDevice = GEngine->GetActiveAudioDevice())
		{
			AudioDevice->StopAllSounds(true);
		}

		// Subtract the current time from our start time and add it to our running total.
		// This allows us to keep track of how much recording has actually been done, not counting paused time.
		TotalGameRecordingTime += GetWorld()->TimeSeconds - GameRecordingStartTime;
		TotalPlatformRecordingTime += FPlatformTime::Seconds() - PlatformRecordingStartTime;

		UE_LOGF(LogTemp, Warning, "Audio Recording Paused. Adding: %f seconds to GameRecording. Adding: %f seconds to Platform Recording.",
			GetWorld()->TimeSeconds - GameRecordingStartTime, FPlatformTime::Seconds() - PlatformRecordingStartTime);

		GameRecordingStartTime = -1.0;
		PlatformRecordingStartTime = -1.0;
	}

	virtual void BeginFinalizeImpl() override
	{
		if (GetWorld() && GetWorld()->GetGameViewport())
		{
			// Re-enable rendering
			GetWorld()->GetGameViewport()->bDisableWorldRendering = false;
		}

		// Convert it to absolute as the Audio Recorder wants to save relative to a different directory.
		FString FormattedFileName = CaptureHost->ResolveFileFormat(FileName, FFrameMetrics());
		FString AbsoluteDirectory = FPaths::ConvertRelativePathToFull(CaptureHost->GetSettings().OutputDirectory.Path);
		UAudioMixerBlueprintLibrary::StopRecordingOutput(GetWorld(), EAudioRecordingExportType::WavFile, FormattedFileName, AbsoluteDirectory);
								
		// Now we can compare the two to see how close they are to each other to try and warn users about potential de-syncs caused by rendering.
		double Difference = TotalGameRecordingTime - TotalPlatformRecordingTime;
		if(!FMath::IsNearlyZero(Difference, 0.05))
		{
			// @todo-sequencer: This doesn't seem to correctly calculate the difference between UWorld time and platform time. It will report only a ~0.1s offset, but the wav file is ~28 seconds longer (platform time).
			UE_LOGF(LogMovieSceneCapture, Warning, "Game Time is out of sync with Platform Time during audio recording. This is usually an indication that the sequence could not play back at full speed, and audio will most likely be desynchronized. Platform Time took %f seconds longer than Game Time.", Difference);
		}
	}
	/**~ End UMovieSceneCaptureProtocolBase Implementation */

protected:
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Audio Options")
	FString FileName;

	double TotalGameRecordingTime;
	double TotalPlatformRecordingTime;
	double GameRecordingStartTime;
	double PlatformRecordingStartTime;
	bool bHasSetup;
};
