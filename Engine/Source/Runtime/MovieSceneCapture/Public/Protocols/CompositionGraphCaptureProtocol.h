// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "MovieSceneCaptureProtocolBase.h"
#include "CompositionGraphCaptureProtocol.generated.h"

class FSceneViewport;
class UMaterialInterface;

struct FMovieSceneCaptureSettings;
struct FFrameCaptureViewExtension;

/** Used by UCompositionGraphCaptureSettings. Matches gamut order in EDisplayColorGamut */
UENUM(BlueprintType)
enum EHDRCaptureGamut : int
{
	HCGM_Rec709 UMETA(DisplayName = "Rec.709 / sRGB"),
	HCGM_P3DCI UMETA(DisplayName = "P3 D65"),
	HCGM_Rec2020 UMETA(DisplayName = "Rec.2020"),
	HCGM_ACES UMETA(DisplayName = "ACES"),
	HCGM_ACEScg UMETA(DisplayName = "ACEScg"),
	HCGM_Linear UMETA(DisplayName = "Linear"),
	HCGM_MAX,
};

static_assert(HCGM_Rec709 == (int32)EDisplayColorGamut::sRGB_D65, "EHDRCaptureGamut and EDisplayColorGamut not matching");
static_assert(HCGM_P3DCI == (int32)EDisplayColorGamut::DCIP3_D65, "EHDRCaptureGamut and EDisplayColorGamut not matching");
static_assert(HCGM_Rec2020 == (int32)EDisplayColorGamut::Rec2020_D65, "EHDRCaptureGamut and EDisplayColorGamut not matching");
static_assert(HCGM_ACES == (int32)EDisplayColorGamut::ACES_D60, "EHDRCaptureGamut and EDisplayColorGamut not matching");
static_assert(HCGM_ACEScg == (int32)EDisplayColorGamut::ACEScg_D60, "EHDRCaptureGamut and EDisplayColorGamut not matching");
// HCGM_Linear gets remapped to DCIP3_D65 internally
//static_assert(HCGM_Linear == (int32)EDisplayColorGamut::DCIP3_D65, "EHDRCaptureGamut and EDisplayColorGamut not matching")
static_assert(HCGM_MAX == (int32)EDisplayColorGamut::MAX + 1, "EHDRCaptureGamut and EDisplayColorGamut not matching");

USTRUCT(BlueprintType)
struct FCompositionGraphCapturePasses
{
	GENERATED_BODY()

	/** List of passes to record by name. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Composition Graph Settings")
	TArray<FString> Value;
};

UCLASS(config=EditorPerProjectUserSettings, meta=(DisplayName="Custom Render Passes", CommandLineID="CustomRenderPasses"), MinimalAPI)
class UCompositionGraphCaptureProtocol : public UMovieSceneImageCaptureProtocolBase
{
public:
	GENERATED_BODY()

	UCompositionGraphCaptureProtocol(const FObjectInitializer& Init)
		: Super(Init)
		, bDisableScreenPercentage(true)
	{}

	/** A list of render passes to include in the capture. Leave empty to export all available passes. */
	UPROPERTY(config, BlueprintReadWrite, EditAnywhere, Category="Composition Graph Options")
	FCompositionGraphCapturePasses IncludeRenderPasses;

	/** Whether to capture the frames as HDR textures (*.exr format) */
	UPROPERTY(config, BlueprintReadWrite, EditAnywhere, Category="Composition Graph Options")
	bool bCaptureFramesInHDR;

	/** Compression Quality for HDR Frames (0 for no compression, 1 for default compression which can be slow) */
	UPROPERTY(config, BlueprintReadWrite, EditAnywhere, Category="Composition Graph Options", meta = (EditCondition = "bCaptureFramesInHDR", UIMin=0, ClampMin=0, UIMax=1, ClampMax=1))
	int32 HDRCompressionQuality;

	/** The color gamut to use when storing HDR captured data. The gamut depends on whether the bCaptureFramesInHDR option is enabled. */
	UPROPERTY(config, BlueprintReadWrite, EditAnywhere, Category="Composition Graph Options", meta = (EditCondition = "bCaptureFramesInHDR"))
	TEnumAsByte<enum EHDRCaptureGamut> CaptureGamut;

	/** Custom post processing material to use for rendering */
	UPROPERTY(config, BlueprintReadWrite, EditAnywhere, Category="Composition Graph Options", meta=(AllowedClasses=""))
	FSoftObjectPath PostProcessingMaterial;

	/** Whether to disable screen percentage */
	UPROPERTY(config, BlueprintReadWrite, EditAnywhere, Category="Composition Graph Options")
	bool bDisableScreenPercentage;

public:

	/**~ UMovieSceneCaptureProtocolBase implementation */
	MOVIESCENECAPTURE_API virtual bool SetupImpl();
	MOVIESCENECAPTURE_API virtual void CaptureFrameImpl(const FFrameMetrics& FrameMetrics);
	MOVIESCENECAPTURE_API virtual void TickImpl() override;
	MOVIESCENECAPTURE_API virtual void FinalizeImpl() override;
	MOVIESCENECAPTURE_API virtual bool HasFinishedProcessingImpl() const override;
	MOVIESCENECAPTURE_API virtual void OnReleaseConfigImpl(FMovieSceneCaptureSettings& InSettings) override;
	MOVIESCENECAPTURE_API virtual void OnLoadConfigImpl(FMovieSceneCaptureSettings& InSettings) override;
	/**~ End UMovieSceneCaptureProtocolBase implementation */

private:

	UPROPERTY(transient)
	TObjectPtr<UMaterialInterface> PostProcessingMaterialPtr;

	/** The viewport we are capturing from */
	TWeakPtr<FSceneViewport> SceneViewport;

	/** A view extension that we use to ensure we dump out the composition graph frames with the correct settings */
	TSharedPtr<FFrameCaptureViewExtension, ESPMode::ThreadSafe> ViewExtension;
};
