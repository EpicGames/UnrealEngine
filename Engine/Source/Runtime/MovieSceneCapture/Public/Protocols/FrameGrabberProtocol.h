// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneCaptureProtocolBase.h"
#include "FrameGrabber.h"

#include "FrameGrabberProtocol.generated.h"


UCLASS(Abstract, config=EditorPerProjectUserSettings, MinimalAPI)
class UFrameGrabberProtocol : public UMovieSceneImageCaptureProtocolBase
{
public:

	GENERATED_BODY()

	UFrameGrabberProtocol(const FObjectInitializer& ObjectInit)
		: Super(ObjectInit)
		, DesiredPixelFormat(PF_B8G8R8A8)
		, RingBufferSize(3)
	{}

	/** The pixel format we want to capture in */
	EPixelFormat DesiredPixelFormat;

	/** The size of the render-target resolution surface ring-buffer */
	uint32 RingBufferSize;

protected:

	/**~ UMovieSceneCaptureProtocolBase Implementation */
	MOVIESCENECAPTURE_API virtual bool HasFinishedProcessingImpl() const override;
	MOVIESCENECAPTURE_API virtual bool SetupImpl() override;
	MOVIESCENECAPTURE_API virtual void CaptureFrameImpl(const FFrameMetrics& FrameMetrics);
	MOVIESCENECAPTURE_API virtual void TickImpl() override;
	MOVIESCENECAPTURE_API virtual void BeginFinalizeImpl() override;
	MOVIESCENECAPTURE_API virtual void FinalizeImpl() override;
	/**~ End UMovieSceneCaptureProtocolBase Implementation */

protected:

	/**
	 * Retrieve an arbitrary set of data that relates to the specified frame metrics.
	 * This data will be passed through the capture pipeline, and will be accessible from ProcessFrame
	 *
	 * @param FrameMetrics	Metrics specific to the current frame
	 * @param Host			The host that is managing this protocol

	 * @return Shared pointer to a payload to associate with the frame, or nullptr
	 */
	virtual FFramePayloadPtr GetFramePayload(const FFrameMetrics& FrameMetrics) PURE_VIRTUAL(UFrameGrabberProtocol::GetFramePayload, return nullptr;)

	/**
	 * Process a captured frame. This may be called on any thread.
	 *
	 * @param Frame			The captured frame data, including any payload retrieved from GetFramePayload
	 */
	virtual void ProcessFrame(FCapturedFrameData Frame) PURE_VIRTUAL(UFrameGrabberProtocol::ProcessFrame,)

private:

	/** The frame grabber, responsible for actually capturing frames */
	TUniquePtr<FFrameGrabber> FrameGrabber;
};
