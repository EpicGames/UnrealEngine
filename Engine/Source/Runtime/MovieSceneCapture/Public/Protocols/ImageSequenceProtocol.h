// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Protocols/FrameGrabberProtocol.h"
#include "IImageWrapper.h"
#include "Async/Future.h"
#include "CompositionGraphCaptureProtocol.h"
#include "ImageSequenceProtocol.generated.h"

struct FMovieSceneCaptureSettings;
class IImageWriteQueue;

UCLASS(Abstract, config=EditorPerProjectUserSettings, MinimalAPI)
class UImageSequenceProtocol : public UFrameGrabberProtocol
{
public:

	GENERATED_BODY()

	MOVIESCENECAPTURE_API UImageSequenceProtocol(const FObjectInitializer& ObjInit);

public:

	/** ~UFrameGrabberProtocol implementation */
	MOVIESCENECAPTURE_API virtual FFramePayloadPtr GetFramePayload(const FFrameMetrics& FrameMetrics);
	MOVIESCENECAPTURE_API virtual void ProcessFrame(FCapturedFrameData Frame);
	/** ~End UFrameGrabberProtocol implementation */

	/** ~ UMovieSceneCaptureProtocol implementation */
	MOVIESCENECAPTURE_API virtual bool SetupImpl() override;
	MOVIESCENECAPTURE_API virtual void AddFormatMappingsImpl(TMap<FString, FStringFormatArg>& FormatMappings) const override;
	MOVIESCENECAPTURE_API virtual void BeginFinalizeImpl() override;
	MOVIESCENECAPTURE_API virtual void FinalizeImpl() override;
	MOVIESCENECAPTURE_API virtual bool HasFinishedProcessingImpl() const override;
	MOVIESCENECAPTURE_API virtual void OnReleaseConfigImpl(FMovieSceneCaptureSettings& InSettings) override;
	MOVIESCENECAPTURE_API virtual void OnLoadConfigImpl(FMovieSceneCaptureSettings& InSettings) override;
	/** ~End UMovieSceneCaptureProtocol implementation */

protected:

	/** The format of the image to write out */
	EImageFormat Format;


private:

	virtual int32 GetCompressionQuality() const { return 0; }

	/** Custom string format arguments for filenames */
	TMap<FString, FStringFormatArg> StringFormatMap;

	/** A pointer to the image write queue used for asynchronously writing images */
	IImageWriteQueue* ImageWriteQueue;

	/** A future that is created on BeginFinalize from a fence in the image write queue that will be fulfilled when all currently pending tasks have been completed */
	TFuture<void> FinalizeFence;
};

UCLASS(config=EditorPerProjectUserSettings, Abstract, MinimalAPI)
class UCompressedImageSequenceProtocol : public UImageSequenceProtocol
{
public:

	GENERATED_BODY()

	/** Level of compression to apply to the image, between 1 (worst quality, best compression) and 100 (best quality, worst compression)*/
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=ImageSettings, meta=(ClampMin=1, ClampMax=100))
	int32 CompressionQuality;

	UCompressedImageSequenceProtocol(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{
		CompressionQuality = 100;
	}

protected:

	virtual int32 GetCompressionQuality() const override { return CompressionQuality; }
	MOVIESCENECAPTURE_API virtual bool SetupImpl() override;
	MOVIESCENECAPTURE_API virtual void AddFormatMappingsImpl(TMap<FString, FStringFormatArg>& FormatMappings) const override;
};

UCLASS(meta=(DisplayName="Image Sequence (bmp)", CommandLineID="BMP"), MinimalAPI)
class UImageSequenceProtocol_BMP : public UImageSequenceProtocol
{
public:

	GENERATED_BODY()

	UImageSequenceProtocol_BMP(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{
		Format = EImageFormat::BMP;
	}
};

UCLASS(meta=(DisplayName="Image Sequence (png)", CommandLineID="PNG"), MinimalAPI)
class UImageSequenceProtocol_PNG : public UCompressedImageSequenceProtocol
{
public:

	GENERATED_BODY()

	UImageSequenceProtocol_PNG(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{
		Format = EImageFormat::PNG;
	}
};

UCLASS(meta=(DisplayName="Image Sequence (jpg)", CommandLineID="JPG"), MinimalAPI)
class UImageSequenceProtocol_JPG : public UCompressedImageSequenceProtocol
{
public:

	GENERATED_BODY()

	UImageSequenceProtocol_JPG(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{
		Format = EImageFormat::JPEG;
	}
};

UCLASS(meta=(DisplayName="Image Sequence (exr)", CommandLineID="EXR"), MinimalAPI)
class UImageSequenceProtocol_EXR : public UImageSequenceProtocol
{
public:

	GENERATED_BODY()

	/** Whether to write out compressed or uncompressed EXRs */
	UPROPERTY(config, EditAnywhere, Category=ImageSettings)
	bool bCompressed;

	/** The color gamut to use when storing HDR captured data. */
	UPROPERTY(config, EditAnywhere, Category=ImageSettings)
	TEnumAsByte<EHDRCaptureGamut> CaptureGamut;

	MOVIESCENECAPTURE_API UImageSequenceProtocol_EXR(const FObjectInitializer& ObjInit);

private:
	virtual int32 GetCompressionQuality() const override { return bCompressed ? (int32)EImageCompressionQuality::Default : (int32)EImageCompressionQuality::Uncompressed; }
	MOVIESCENECAPTURE_API virtual bool SetupImpl() override;
	MOVIESCENECAPTURE_API virtual void FinalizeImpl() override;
	MOVIESCENECAPTURE_API virtual void AddFormatMappingsImpl(TMap<FString, FStringFormatArg>& FormatMappings) const override;

	EDisplayColorGamut RestoreColorGamut;
	EDisplayOutputFormat RestoreOutputDevice;
};

