// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MovieSceneCaptureProtocolBase.h"
#include "FrameGrabber.h"
#include "ImageWriteStream.h"
#include "UserDefinedCaptureProtocol.generated.h"

enum class EDesiredImageFormat : uint8;
struct FImagePixelData;
struct FImageWriteOptions;
struct ICaptureProtocolHost;
class UTexture;

/** Structure used as an identifier for a particular buffer within a capture. */
USTRUCT(BlueprintType)
struct FCapturedPixelsID
{
	GENERATED_BODY()

	/** Convert this strream ID to a string */
	FString ToString() const;

	/** Map of identifiers to their values for this ID. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Capture)
	TMap<FName, FName> Identifiers;
};



USTRUCT(BlueprintType)
struct FCapturedPixels
{
	GENERATED_BODY()

	TSharedPtr<FImagePixelData, ESPMode::ThreadSafe> ImageData;
};

/**
 * A blueprintable capture protocol that defines how to capture frames from the engine
 */
UCLASS(Abstract, Blueprintable, meta=(DisplayName="Capture Protocol"), MinimalAPI)
class UUserDefinedCaptureProtocol
	: public UMovieSceneImageCaptureProtocolBase
{
public:

	GENERATED_BODY()

	MOVIESCENECAPTURE_API UUserDefinedCaptureProtocol(const FObjectInitializer& ObjInit);

protected:

	/**
	 * Called before the capture process itself is ticked, before each frame is set up for capture
	 * Useful for any pre-frame set up or resetting the previous frame's state
	 */
	UFUNCTION(BlueprintImplementableEvent, Category=Capture)
	MOVIESCENECAPTURE_API void OnPreTick();

	/**
	 * Called after the capture process itself is ticked, after the frame is set up for capture, but before most actors have ticked
	 */
	UFUNCTION(BlueprintImplementableEvent, Category=Capture)
	MOVIESCENECAPTURE_API void OnTick();

	/**
	 * Called once at the start of the capture process, but before any warmup frames
	 * @return true if this protocol set up successfully, false otherwise
	 */
	UFUNCTION(BlueprintNativeEvent, Category=Capture)
	MOVIESCENECAPTURE_API bool OnSetup();
	virtual bool OnSetup_Implementation() { return true; }

	/**
	 * Called when the capture process is warming up. Protocols may transition from either an initialized, or capturing state to warm-up
	 */
	UFUNCTION(BlueprintImplementableEvent, Category=Capture)
	MOVIESCENECAPTURE_API void OnWarmUp();

	/**
	 * Called when this protocol should start capturing frames
	 */
	UFUNCTION(BlueprintImplementableEvent, Category=Capture)
	MOVIESCENECAPTURE_API void OnStartCapture();

	/**
	 * Called when this protocol should capture the current frame
	 */
	UFUNCTION(BlueprintImplementableEvent, Category=Capture)
	MOVIESCENECAPTURE_API void OnCaptureFrame();

	/**
	 * Called when this protocol should temporarily stop capturing frames
	 */
	UFUNCTION(BlueprintImplementableEvent, Category=Capture)
	MOVIESCENECAPTURE_API void OnPauseCapture();

	/**
	 * Called when this protocol is going to be shut down - should not capture any more frames
	 */
	UFUNCTION(BlueprintImplementableEvent, Category=Capture)
	MOVIESCENECAPTURE_API void OnBeginFinalize();


	/**
	 * Called to check whether this protocol has finished any pending tasks, and can now be shut down
	 */
	UFUNCTION(BlueprintNativeEvent, Category=Capture)
	MOVIESCENECAPTURE_API bool OnCanFinalize() const;
	virtual bool OnCanFinalize_Implementation() const { return true; }


	/**
	 * Called to complete finalization of this protocol
	 */
	UFUNCTION(BlueprintImplementableEvent, Category=Capture)
	MOVIESCENECAPTURE_API void OnFinalize();


	/**
	 * Called when pixels have been received for the specified stream name
	 */
	UFUNCTION(BlueprintImplementableEvent, Category=Capture)
	MOVIESCENECAPTURE_API void OnPixelsReceived(const FCapturedPixels& Pixels, const FCapturedPixelsID& ID, FFrameMetrics FrameMetrics);


public:

	/*
	 * Resolve the specified buffer and pass it directly to the specified handler when done (does not pass to any bound streams)
	 * 
	 * @param Buffer          The desired buffer to save
	 * @param BufferID        The ID of this buffer that is passed to the pixel handler (e.g. a composition pass name).
	 */
	UFUNCTION(BlueprintCallable, Category=Capture)
	MOVIESCENECAPTURE_API void ResolveBuffer(UTexture* Buffer, const FCapturedPixelsID& BufferID);


	/**
	 * Instruct this protocol to start capturing LDR final pixels (including slate widgets and burn-ins)
	 *
	 * @param StreamID        The identifier to use for the final pixels buffer
	 */
	UFUNCTION(BlueprintCallable, Category=Capture)
	MOVIESCENECAPTURE_API void StartCapturingFinalPixels(const FCapturedPixelsID& StreamID);


	/**
	 * Instruct this protocol to stop capturing LDR final pixels
	 */
	UFUNCTION(BlueprintCallable, Category=Capture)
	MOVIESCENECAPTURE_API void StopCapturingFinalPixels();


	/**
	 * Generate a filename for the current frame
	 */
	UFUNCTION(BlueprintCallable, Category=Capture)
	MOVIESCENECAPTURE_API virtual FString GenerateFilename(const FFrameMetrics& InFrameMetrics) const;


	/**
	 * Access this protocol's current frame metrics
	 */
	UFUNCTION(BlueprintCallable, Category=Capture)
	FFrameMetrics GetCurrentFrameMetrics() const
	{
		return CachedFrameMetrics;
	}

public:

	/**
	 * Called when image pixel data is ready to be processed
	 */
	MOVIESCENECAPTURE_API void OnPixelsReceivedImpl(const FCapturedPixels& Pixels, const FCapturedPixelsID& StreamID, FFrameMetrics FrameMetrics);

	/**
	 * INTERNAL: Report that the protocol is dispatching the specified number of asynchronous tasks that need to be completed
	 * before this protocol can be finalized
	 */
	MOVIESCENECAPTURE_API void ReportOutstandingWork(int32 NumNewOperations);

protected:

	/**~ Begin UMovieSceneCaptureProtocolBase implementation */
	MOVIESCENECAPTURE_API virtual void PreTickImpl() override;
	MOVIESCENECAPTURE_API virtual void TickImpl() override;
	MOVIESCENECAPTURE_API virtual bool SetupImpl() override;
	MOVIESCENECAPTURE_API virtual void WarmUpImpl() override;
	MOVIESCENECAPTURE_API virtual bool StartCaptureImpl() override;
	MOVIESCENECAPTURE_API virtual void CaptureFrameImpl(const FFrameMetrics& FrameMetrics) override;
	MOVIESCENECAPTURE_API virtual void BeginFinalizeImpl() override;
	MOVIESCENECAPTURE_API virtual bool HasFinishedProcessingImpl() const override;
	MOVIESCENECAPTURE_API virtual void FinalizeImpl() override;
	MOVIESCENECAPTURE_API virtual void AddFormatMappingsImpl(TMap<FString, FStringFormatArg>& FormatMappings) const override;
	/**~ End UMovieSceneCaptureProtocolBase implementation */

	/**~ Begin UObject implementation */
	virtual UWorld* GetWorld() const override { return World; }
	/**~ End UObject implementation */

protected:

	/** World pointer assigned on Setup */
	UPROPERTY(transient, BlueprintReadOnly, Category=Capture)
	TObjectPtr<UWorld> World;

protected:

	/** A frame grabber responsible for capturing LDR final pixels from the viewport when requested */
	TUniquePtr<FFrameGrabber> FinalPixelsFrameGrabber;

	/** A running count of the number of currently pending async operations */
	TAtomic<int32> NumOutstandingOperations;

	/** Frame metrics cached for the current frame */
	FFrameMetrics CachedFrameMetrics;

	/** The ID of the final pixel stream cached from StartCapturingFinalPixels */
	FCapturedPixelsID FinalPixelsID;

	/** Transient stream ID used only during filename generation */
	const FCapturedPixelsID* CurrentStreamID;
};


/**
 * A blueprintable capture protocol tailored to capturing and exporting frames as images
 */
UCLASS(Abstract, config=EditorPerProjectUserSettings, Blueprintable, meta=(DisplayName="Capture Protocol"), MinimalAPI)
class UUserDefinedImageCaptureProtocol
	: public UUserDefinedCaptureProtocol
{
public:

	GENERATED_BODY()

	MOVIESCENECAPTURE_API UUserDefinedImageCaptureProtocol(const FObjectInitializer& ObjInit);

	/** The image format to save as */
	UPROPERTY(config, BlueprintReadWrite, EditAnywhere, Category=Capture)
	EDesiredImageFormat Format;

	/** Whether to save images with compression or not. Not supported for bitmaps. */
	UPROPERTY(config, BlueprintReadWrite, EditAnywhere, Category=Capture, meta=(InlineEditConditionToggle))
	bool bEnableCompression;

	/** The compression quality for the image type. For EXRs, 0 = Default ZIP compression, 1 = No compression, PNGs and JPEGs expect a value between 0 and 100*/
	UPROPERTY(config, BlueprintReadWrite, EditAnywhere, Category=Capture, meta=(EditCondition=bEnableCompression))
	int32 CompressionQuality;

public:


	/*
	 * Generate a filename for the specified buffer using this protocol's file name formatter
	 * 
	 * @param Buffer          The desired buffer to generate a filename for
	 * @param StreamID        The ID of the stream for this buffer (e.g. a composition pass name)
	 * @return A fully qualified file name
	 */
	UFUNCTION(BlueprintCallable, Category=Capture)
	MOVIESCENECAPTURE_API FString GenerateFilenameForBuffer(UTexture* Buffer, const FCapturedPixelsID& StreamID);


	/*
	 * Generate a filename for the current frame using this protocol's file name formatter
	 * 
	 * @return A fully qualified file name for the current frame number
	 */
	UFUNCTION(BlueprintCallable, Category=Capture)
	MOVIESCENECAPTURE_API FString GenerateFilenameForCurrentFrame();


	/*
	 * Generate a filename for the current frame using this protocol's file name formatter
	 * 
	 * @return A fully qualified file name for the current frame number
	 */
	UFUNCTION(BlueprintCallable, Category=Capture)
	MOVIESCENECAPTURE_API void WriteImageToDisk(const FCapturedPixels& PixelData, const FCapturedPixelsID& StreamID, const FFrameMetrics& FrameMetrics, bool bCopyImageData = false);

protected:

	/**~ Begin UMovieSceneCaptureProtocolBase implementation */
	MOVIESCENECAPTURE_API virtual void OnReleaseConfigImpl(FMovieSceneCaptureSettings& InSettings) override;
	MOVIESCENECAPTURE_API virtual void OnLoadConfigImpl(FMovieSceneCaptureSettings& InSettings) override;
	/**~ End UMovieSceneCaptureProtocolBase implementation */

public:

	/**
	 * Called on the main thread when an async operation dispatched from this class has completed (either successfully or otherwise)
	 */
	MOVIESCENECAPTURE_API void OnFileWritten();

private:

	/**~ Begin UUserDefinedCaptureProtocol implementation */
	MOVIESCENECAPTURE_API virtual FString GenerateFilename(const FFrameMetrics& InFrameMetrics) const override;
	/**~ Begin UUserDefinedCaptureProtocol implementation */
};
