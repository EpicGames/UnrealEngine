// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneCaptureProtocolBase.h"
#include "FrameGrabber.h"
#include "Protocols/FrameGrabberProtocol.h"
#include "AVIWriter.h"
#include "VideoCaptureProtocol.generated.h"


UCLASS(meta=(DisplayName="Video Sequence (avi)", CommandLineID="Video"), MinimalAPI)
class UVideoCaptureProtocol : public UFrameGrabberProtocol
{
public:
	GENERATED_BODY()

	UVideoCaptureProtocol(const FObjectInitializer& Init)
		: Super(Init)
		, bUseCompression(true)
		, CompressionQuality(75)
	{}

public:

	UPROPERTY(config, EditAnywhere, Category=VideoSettings)
	bool bUseCompression;

	UPROPERTY(config, EditAnywhere, Category=VideoSettings, meta=(ClampMin=1, ClampMax=100, EditCondition=bUseCompression))
	float CompressionQuality;

public:
	MOVIESCENECAPTURE_API virtual bool SetupImpl() override;
	MOVIESCENECAPTURE_API virtual void FinalizeImpl() override;
	MOVIESCENECAPTURE_API virtual FFramePayloadPtr GetFramePayload(const FFrameMetrics& FrameMetrics);
	MOVIESCENECAPTURE_API virtual void ProcessFrame(FCapturedFrameData Frame);
	MOVIESCENECAPTURE_API virtual bool CanWriteToFileImpl(const TCHAR* InFilename, bool bOverwriteExisting) const override;

protected:

	MOVIESCENECAPTURE_API void ConditionallyCreateWriter();
private:

	TArray<TUniquePtr<FAVIWriter>> AVIWriters;
};
